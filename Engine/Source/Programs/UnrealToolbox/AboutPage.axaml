<UserControl xmlns="https://github.com/avaloniaui"
             xmlns:x="http://schemas.microsoft.com/winfx/2006/xaml"
             xmlns:d="http://schemas.microsoft.com/expression/blend/2008"
             xmlns:mc="http://schemas.openxmlformats.org/markup-compatibility/2006"
             xmlns:ui="using:FluentAvalonia.UI.Controls"
             xmlns:local="using:UnrealToolbox"
             mc:Ignorable="d" d:DesignWidth="800" d:DesignHeight="600"
             x:Class="UnrealToolbox.AboutPage"
             Background="{DynamicResource SolidBackgroundFillColorBaseBrush}">

	<UserControl.Styles>
		<StyleInclude Source="/Styles.axaml" />
	</UserControl.Styles>

	<ScrollViewer>
		<StackPanel Margin="20" Spacing="25">
			<TextBlock Theme="{StaticResource SubtitleTextBlockStyle}" VerticalAlignment="Center" Margin="0 0 0 0">About</TextBlock>

			<Grid>
				<Grid.ColumnDefinitions>
					<ColumnDefinition Width="Auto" MinWidth="180" />
					<ColumnDefinition Width="*" />
				</Grid.ColumnDefinitions>
				<Grid.RowDefinitions>
					<RowDefinition Height="Auto" />
					<RowDefinition Height="Auto" />
					<RowDefinition Height="Auto" />
				</Grid.RowDefinitions>

				<!-- Labels column -->
				<TextBlock Grid.Row="0" Grid.Column="0" Text="Version:" VerticalAlignment="Center" />
				<TextBlock Grid.Row="1" Grid.Column="0" Text="Settings and data directory:" VerticalAlignment="Center" />
				<TextBlock Grid.Row="2" Grid.Column="0" Text="Log file:" VerticalAlignment="Center" />

				<!-- Values column -->
				<TextBlock Grid.Row="0" Grid.Column="1" Name="_versionText" VerticalAlignment="Center" Padding="11,5,11,6" />
				<HyperlinkButton Grid.Column="1" Grid.Row="1" FontSize="14" VerticalAlignment="Center" Name="_settingsDirButton"/>
				<HyperlinkButton Grid.Column="1" Grid.Row="2" FontSize="14" VerticalAlignment="Center" Name="_logFileButton"/>
			</Grid>
		</StackPanel>
	</ScrollViewer>

</UserControl>