// Copyright Epic Games, Inc. All Rights Reserved.

using System.Diagnostics;
using System.Runtime.InteropServices;
using Avalonia.Controls;

namespace UnrealToolbox
{
	partial class AboutPage : UserControl
	{
		public AboutPage()
		{
			InitializeComponent();

			string version = Program.Update?.CurrentVersionString ?? "No version information present.";
			_versionText.Text = version;
			
			string settingsDir = Program.DataDir.FullName;
			_settingsDirButton.Click += (sender, args) => OpenFolder(settingsDir);
			_settingsDirButton.Content = settingsDir;
			
			if (Program.LogFile != null)
			{
				string logFile = Program.LogFile.FullName;
				_logFileButton.Click += (sender, args) => OpenFile(logFile);
				_logFileButton.Content = logFile;
			}
		}
		
		private static void OpenFolder(string path)
		{
			if (RuntimeInformation.IsOSPlatform(OSPlatform.Windows))
			{
				Process.Start("explorer.exe", path);
			}
			else if (RuntimeInformation.IsOSPlatform(OSPlatform.Linux))
			{
				Process.Start("xdg-open", path);
			}
			else if (RuntimeInformation.IsOSPlatform(OSPlatform.OSX))
			{
				Process.Start("open", path);
			}
		}
		
		private static void OpenFile(string path)
		{
			if (RuntimeInformation.IsOSPlatform(OSPlatform.Windows))
			{
				Process.Start(new ProcessStartInfo(path) { UseShellExecute = true });
			}
			else if (RuntimeInformation.IsOSPlatform(OSPlatform.Linux))
			{
				Process.Start("xdg-open", path);
			}
			else if (RuntimeInformation.IsOSPlatform(OSPlatform.OSX))
			{
				Process.Start("open", path);
			}
		}
	}
}