<Application xmlns="https://github.com/avaloniaui"
			xmlns:x="http://schemas.microsoft.com/winfx/2006/xaml"
			xmlns:sty="using:FluentAvalonia.Styling"
			x:Class="UnrealToolbox.App"
			Name="Unreal Toolbox"
			RequestedThemeVariant="Default">
			<!-- "Default" ThemeVariant follows system theme variant. "Dark" or "Light" are other available options. -->
	
	<Application.Styles>
		<!-- As of preview5, these properties now default to false -->
		<sty:FluentAvaloniaTheme PreferSystemTheme="True" PreferUserAccentColor="True" />
	</Application.Styles>

	<Application.Resources>
		<WindowIcon x:Key="StatusBusy">/Resources/StatusBusy.ico</WindowIcon>
		<WindowIcon x:Key="StatusError">/Resources/StatusError.ico</WindowIcon>
		<WindowIcon x:Key="StatusNormal">/Resources/StatusNormal.ico</WindowIcon>
		<WindowIcon x:Key="StatusPaused">/Resources/StatusPaused.ico</WindowIcon>
	</Application.Resources>
	
	<TrayIcon.Icons>
		<TrayIcons>
			<TrayIcon Clicked="TrayIcon_Click"/>
		</TrayIcons>
	</TrayIcon.Icons>
	
</Application>