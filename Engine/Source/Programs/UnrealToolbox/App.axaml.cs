// Copyright Epic Games, Inc. All Rights Reserved.

using System.Diagnostics;
using System.Reflection;
using Avalonia;
using Avalonia.Controls;
using Avalonia.Controls.ApplicationLifetimes;
using Avalonia.Markup.Xaml;
using Avalonia.Threading;
using EpicGames.Core;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Logging;
using Microsoft.Win32;
using UnrealToolbox.Plugins.Artifacts;
using UnrealToolbox.Plugins.HordeAgent;
using UnrealToolbox.Plugins.HordeProxy;

namespace UnrealToolbox
{
	/// <summary>
	/// Main application class
	/// </summary>
	public sealed partial class App : Application, IToolboxPluginHost, IAsyncDisposable
	{
		readonly ServiceProvider _serviceProvider;
		readonly ILogger _logger;

		WindowIcon? _normalIcon;
		WindowIcon? _busyIcon;
		WindowIcon? _pausedIcon;
		WindowIcon? _errorIcon;
		DispatcherTimer? _retryUpdateTimer;

		SettingsWindow? _settingsWindow;

		Thread? _settingsThread;
		ManualResetEvent? _settingsThreadStop;

		/// <summary>
		/// Constructor
		/// </summary>
		public App()
		{
			ServiceCollection serviceCollection = new ServiceCollection();
			serviceCollection.AddLogging(builder => builder.AddEpicDefault());
			serviceCollection.AddSingleton<IHordeClientProvider, HordeClientProvider>();
			serviceCollection.AddSingleton<ToolCatalog>();
			serviceCollection.AddSingleton<IToolCatalog, ToolCatalog>(sp => sp.GetRequiredService<ToolCatalog>());
			serviceCollection.AddSingleton<IToolboxPluginHost>(this);
			serviceCollection.AddSingleton<SelfUpdateService>();
			serviceCollection.AddSingleton<ToolboxNotificationManager>();

			// Configure plugins
			serviceCollection.AddSingleton<IToolboxPlugin, ArtifactsPlugin>();
			serviceCollection.AddSingleton<IToolboxPlugin, HordeAgentPlugin>();
			serviceCollection.AddSingleton<IToolboxPlugin, HordeProxyPlugin>();

			_serviceProvider = serviceCollection.BuildServiceProvider();
			_logger = _serviceProvider.GetRequiredService<ILogger<App>>();
		}

		/// <inheritdoc/>
		public async ValueTask DisposeAsync()
		{
			if (_settingsThreadStop != null)
			{
				_settingsThreadStop.Set();
				_settingsThread?.Join();
				_settingsThreadStop.Dispose();
				_settingsThreadStop = null;
			}
			await _serviceProvider.DisposeAsync();
		}

		/// <inheritdoc/>
		public override void Initialize()
		{
			AvaloniaXamlLoader.Load(this);

			_normalIcon = (WindowIcon)Resources["StatusNormal"]!;
			_busyIcon = (WindowIcon)Resources["StatusBusy"]!;
			_pausedIcon = (WindowIcon)Resources["StatusPaused"]!;
			_errorIcon = (WindowIcon)Resources["StatusError"]!;

			RefreshPlugins();
			UpdateMenu();

			ToolCatalog toolCatalog = _serviceProvider.GetRequiredService<ToolCatalog>();
			toolCatalog.OnItemsChanged += UpdateMenu;
			toolCatalog.Start();

			SelfUpdateService selfUpdateService = _serviceProvider.GetRequiredService<SelfUpdateService>();
			selfUpdateService.OnUpdateReady += UpdateReady;
			selfUpdateService.Start();

			ToolboxNotificationManager notificationManager = _serviceProvider.GetRequiredService<ToolboxNotificationManager>();
			notificationManager.Start();

			_settingsThreadStop = new ManualResetEvent(false);
			_settingsThread = new Thread(WaitForEvents);
			_settingsThread.Start();

			RegisterProtocolHandlers();
		}

		void RegisterProtocolHandlers()
		{
			List<string> protocols = new List<string>();
			foreach (IToolboxPlugin plugin in _serviceProvider.GetServices<IToolboxPlugin>())
			{
				protocols.AddRange(plugin.UrlProtocols);
			}

			if (OperatingSystem.IsWindows())
			{
				string exeFile = Path.ChangeExtension(Assembly.GetExecutingAssembly().Location, ".exe");
				foreach (string protocol in protocols)
				{
					using (RegistryKey baseKey = Registry.CurrentUser.CreateSubKey("SOFTWARE\\Classes\\horde-artifact"))
					{
						baseKey.SetValue(null, $"URL:{protocol} protocol");
						baseKey.SetValue("URL Protocol", "");

						using (RegistryKey iconKey = baseKey.CreateSubKey("DefaultIcon"))
						{
							iconKey.SetValue(null, $"\"{exeFile}\",0");
						}

						using (RegistryKey commandKey = baseKey.CreateSubKey("shell\\open\\command"))
						{
							commandKey.SetValue(null, $"\"{exeFile}\" -url=\"%1\" -quiet");
						}
					}
				}
			}
		}

		private void WaitForEvents()
		{
			if (!OperatingSystem.IsWindows())
			{
				return; // Named global events are not currently supported on other platforms
			}

			using EventWaitHandle closeEvent = new EventWaitHandle(false, EventResetMode.AutoReset, Program.CloseEventName);
			using EventWaitHandle refreshEvent = new EventWaitHandle(false, EventResetMode.AutoReset, Program.RefreshEventName);
			using EventWaitHandle settingsEvent = new EventWaitHandle(false, EventResetMode.AutoReset, Program.SettingsEventName);
			using EventWaitHandle commandEvent = new EventWaitHandle(false, EventResetMode.AutoReset, Program.CommandEventName);
			for (; ; )
			{
				int index = WaitHandle.WaitAny(new[] { closeEvent, refreshEvent, settingsEvent, commandEvent, _settingsThreadStop! });
				if (index == 0)
				{
					Dispatcher.UIThread.Post(() => CloseMainThread());
				}
				else if (index == 1)
				{
					Dispatcher.UIThread.Post(() => RefreshPlugins());
				}
				else if (index == 2)
				{
					Dispatcher.UIThread.Post(() => OpenSettings());
				}
				else if (index == 3)
				{
					Dispatcher.UIThread.Post(() => ExecuteCommands());
				}
				else
				{
					break;
				}
			}
		}

		void CloseMainThread()
		{
			((IClassicDesktopStyleApplicationLifetime)ApplicationLifetime!).Shutdown();
		}

		void ExecuteCommands()
		{
			List<Command> commands = CommandHelper.DequeueAll();
			foreach (Command command in commands)
			{
				try
				{
					ExecuteCommand(command);
				}
				catch (Exception ex)
				{
					_logger.LogError(ex, "Error executing command: {Message}", ex.Message);
				}
			}
		}

		void ExecuteCommand(Command command)
		{
			if (command.Type == CommandType.OpenUrl)
			{
				Uri url = new Uri(command.Argument!);
				foreach (IToolboxPlugin plugin in _serviceProvider.GetServices<IToolboxPlugin>())
				{
					if (plugin.HandleUrl(url))
					{
						break;
					}
				}
			}
		}

		private void UpdateReady()
		{
			Dispatcher.UIThread.Post(() => UpdateReadyMainThread());
		}

		void UpdateReadyMainThread()
		{
			if (AllWindowsClosed())
			{
				_logger.LogInformation("Shutting down to install update");
				((IClassicDesktopStyleApplicationLifetime)ApplicationLifetime!).Shutdown();
			}
			else if (_retryUpdateTimer == null)
			{
				_logger.LogInformation("Scheduling update when settings window closes");
				_retryUpdateTimer = new DispatcherTimer(TimeSpan.FromSeconds(30.0), DispatcherPriority.Default, (_, _) => UpdateReadyMainThread());
			}
		}

		bool AllWindowsClosed()
		{
			IClassicDesktopStyleApplicationLifetime? lifetime = ApplicationLifetime as IClassicDesktopStyleApplicationLifetime;
			if (lifetime != null && lifetime.Windows.Count == 0)
			{
				return true;
			}
			else
			{
				return false;
			}
		}

		private void UpdateMenu()
		{
			NativeMenuItem settingsMenuItem = new NativeMenuItem("Settings...");
			settingsMenuItem.Click += TrayIcon_Settings;

			NativeMenuItem exitMenuItem = new NativeMenuItem("Exit");
			exitMenuItem.Click += TrayIcon_Exit;

			NativeMenu contextMenu = new NativeMenu();

			TrayIcon trayIcon = TrayIcon.GetIcons(this)![0];
			trayIcon.Menu = contextMenu;

			foreach (IToolboxPlugin plugin in _serviceProvider.GetServices<IToolboxPlugin>())
			{
				plugin.PopulateContextMenu(contextMenu);
			}

			int numItems = contextMenu.Items.Count;
			if (numItems > 0)
			{
				contextMenu.Items.Add(new NativeMenuItemSeparator());
				numItems++;
			}

			IToolCatalog toolCatalog = _serviceProvider.GetRequiredService<IToolCatalog>();
			foreach (IToolCatalogItem item in toolCatalog.Items)
			{
				CurrentToolDeploymentInfo? current = item.Current;
				if (current != null)
				{
					ToolConfig? toolConfig = current.Config;
					if (toolConfig != null && toolConfig.PopupMenu != null)
					{
						AddMenuItems(item, toolConfig.PopupMenu, contextMenu);
					}
				}
			}

			if (contextMenu.Items.Count > numItems)
			{
				contextMenu.Items.Add(new NativeMenuItemSeparator());
			}
			contextMenu.Items.Add(settingsMenuItem);
			contextMenu.Items.Add(exitMenuItem);
		}

		static void AddMenuItems(IToolCatalogItem item, ToolMenuItem toolMenuItem, NativeMenu menu)
		{
			if (!String.IsNullOrEmpty(toolMenuItem.Label))
			{
				NativeMenuItem menuItem = new NativeMenuItem(toolMenuItem.Label);
				if (toolMenuItem.Children != null && toolMenuItem.Children.Count > 0)
				{
					NativeMenu subMenu = new NativeMenu();
					foreach (ToolMenuItem child in toolMenuItem.Children)
					{
						AddMenuItems(item, child, subMenu);
					}
					menuItem.Menu = subMenu;
				}
				else
				{
					if (toolMenuItem.Command != null)
					{
						menuItem.Click += (_, _) => TrayIcon_RunCommand(item, toolMenuItem.Command);
					}
				}
				menu.Items.Add(menuItem);
			}
		}

		static void TrayIcon_RunCommand(IToolCatalogItem item, ToolCommand command)
		{
			ProcessStartInfo startInfo = new ProcessStartInfo(command.FileName);
			if (command.Arguments != null)
			{
				foreach (string argument in command.Arguments)
				{
					startInfo.ArgumentList.Add(argument);
				}
			}
			startInfo.WorkingDirectory = item.Current!.Dir.FullName;
			Process.Start(startInfo);
		}

		private void RefreshPlugins()
		{
			bool update = false;
			foreach (IToolboxPlugin plugin in _serviceProvider.GetServices<IToolboxPlugin>())
			{
				update |= plugin.Refresh();
			}
			if (update)
			{
				UpdateMenu();
				_settingsWindow?.Refresh();
			}
		}

		private void TrayIcon_Click(object? sender, EventArgs e)
		{
			RefreshPlugins();
			OpenSettings();
		}

		private void TrayIcon_Settings(object? sender, EventArgs e)
		{
			OpenSettings();
		}

		private void OpenSettings()
		{
			if (_settingsWindow == null)
			{
				_settingsWindow = new SettingsWindow(_serviceProvider);
				_settingsWindow.Closed += SettingsWindow_Closed;
			}

			Program.BringWindowToFront(_settingsWindow);
		}

		private void SettingsWindow_Closed(object? sender, EventArgs e)
		{
			_settingsWindow = null;
		}

		private void TrayIcon_Exit(object? sender, EventArgs e)
		{
			((IClassicDesktopStyleApplicationLifetime)ApplicationLifetime!).Shutdown();
		}

		/// <inheritdoc/>
		public void UpdateStatus()
		{
			Dispatcher.UIThread.Post(() => UpdateStatusMainThread());
		}

		private void UpdateStatusMainThread()
		{
			TrayAppPluginState state = TrayAppPluginState.Undefined;
			List<string> messages = new List<string>();

			foreach (IToolboxPlugin plugin in _serviceProvider.GetServices<IToolboxPlugin>())
			{
				ToolboxPluginStatus status = plugin.GetStatus();

				if (status.State >= state)
				{
					if (status.State > state)
					{
						messages.Clear();
						state = status.State;
					}
					if (status.Message != null)
					{
						messages.Add(status.Message);
					}
				}
			}

			TrayIcon trayIcon = TrayIcon.GetIcons(this)![0];
			trayIcon!.Icon = state switch
			{
				TrayAppPluginState.Busy => _busyIcon,
				TrayAppPluginState.Paused => _pausedIcon,
				TrayAppPluginState.Error => _errorIcon,
				_ => _normalIcon
			};
			trayIcon.ToolTipText = String.Join("\n", messages);
		}
	}
}
