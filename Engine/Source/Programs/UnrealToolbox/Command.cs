// Copyright Epic Games, Inc. All Rights Reserved.

using System.Text.Json;
using System.Text.Json.Serialization;
using EpicGames.Core;

namespace UnrealToolbox
{
	[JsonConverter(typeof(JsonStringEnumConverter))]
	enum CommandType
	{
		OpenUrl,
	}

	class Command
	{
		public CommandType Type { get; set; }
		public string? Argument { get; set; }
	}

	class CommandList
	{
		public List<Command> Commands { get; set; } = new List<Command>();
	}

	class CommandHelper
	{
		const string MutexName = "UnrealToolbox-CommandMutex";

		static readonly FileReference s_commandFile = FileReference.Combine(Program.DataDir, "Commands.json");

		static readonly JsonSerializerOptions s_jsonOptions = new JsonSerializerOptions
		{
			AllowTrailingCommas = true,
			DefaultIgnoreCondition = JsonIgnoreCondition.WhenWritingNull,
			PropertyNameCaseInsensitive = true,
			PropertyNamingPolicy = JsonNamingPolicy.CamelCase,
			WriteIndented = true
		};

		public static void Enqueue(Command command)
		{
			RunUnderMutex(() =>
			{
				CommandList commandList = ReadCommandList();
				commandList.Commands.Add(command);
				WriteCommandList(commandList);
			});
		}

		public static List<Command> DequeueAll()
		{
			CommandList? commandList = null;
			RunUnderMutex(() =>
			{
				commandList = ReadCommandList();
				FileReference.Delete(s_commandFile);
			});
			return commandList?.Commands ?? new List<Command>();
		}

		static void RunUnderMutex(Action action)
		{
			using Mutex mutex = new Mutex(false, MutexName);
			try
			{
				mutex.WaitOne();
			}
			catch (AbandonedMutexException)
			{
				// Ignore
			}

			try
			{
				action();
			}
			finally
			{
				mutex.ReleaseMutex();
			}
		}

		static CommandList ReadCommandList()
		{
			CommandList? commandList = null;
			using (FileStream? stream = FileTransaction.OpenRead(s_commandFile))
			{
				if (stream == null)
				{
					commandList = new CommandList();
				}
				else
				{
					try
					{
						commandList = JsonSerializer.Deserialize<CommandList>(stream, s_jsonOptions) ?? new CommandList();
					}
					catch
					{
						commandList = new CommandList();
					}
				}
			}
			return commandList;
		}

		static void WriteCommandList(CommandList commandList)
		{
			using (FileTransactionStream? stream = FileTransaction.OpenWrite(s_commandFile))
			{
				JsonSerializer.Serialize(stream, commandList, s_jsonOptions);
				stream.CompleteTransaction();
			}
		}
	}
}
