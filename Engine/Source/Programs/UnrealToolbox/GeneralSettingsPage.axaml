<UserControl xmlns="https://github.com/avaloniaui"
			xmlns:x="http://schemas.microsoft.com/winfx/2006/xaml"
			xmlns:d="http://schemas.microsoft.com/expression/blend/2008"
			xmlns:mc="http://schemas.openxmlformats.org/markup-compatibility/2006"
			xmlns:ui="using:FluentAvalonia.UI.Controls"
			xmlns:local="using:UnrealToolbox"
			mc:Ignorable="d" d:DesignWidth="800" d:DesignHeight="600"
			x:Class="UnrealToolbox.GeneralSettingsPage"
			x:DataType="local:GeneralSettingsViewModel"
			Background="{DynamicResource SolidBackgroundFillColorBaseBrush}">

	<UserControl.Styles>
		<StyleInclude Source="/Styles.axaml"/>
	</UserControl.Styles>
	
	<ScrollViewer>
		<StackPanel Margin="20" Spacing="5">

			<TextBlock Theme="{StaticResource SubtitleTextBlockStyle}" VerticalAlignment="Center" Margin="0 0 0 0">General</TextBlock>

			<TextBlock Classes="SettingHeader">Horde Server</TextBlock>

			<Border Classes="Card">
				<Grid ColumnDefinitions="*,Auto,Auto,Auto" RowDefinitions="Auto,Auto">
					<HyperlinkButton Grid.Column="0" Grid.Row="0" Padding="0" FontSize="14" Content="{Binding ServerUrl}" Command="{Binding OpenServer}"/>

					<Grid Grid.Column="0" Grid.Row="1" ColumnDefinitions="Auto,*" RowDefinitions="Auto">
						<ui:ProgressRing Grid.Column="0" Grid.Row="0" Width="16" Height="16" Margin="0 2 6 2" IsVisible="{Binding IsConnecting}" VerticalAlignment="Top"/>
						<TextBlock Grid.Column="1" Grid.Row="0" Padding="1" FontSize="12" Text="{Binding ServerStatus}" VerticalAlignment="Top" TextTrimming="CharacterEllipsis" TextWrapping="Wrap"/>
					</Grid>

					<Button Grid.Column="1" Command="{Binding StartRefresh}" IsEnabled="{Binding !IsConnecting}" Margin="0 0 5 0" Grid.RowSpan="2">Refresh</Button>
					<Button Grid.Column="2" Command="{Binding CancelRefresh}" IsEnabled="{Binding IsConnecting}" Margin="0 0 5 0" Grid.RowSpan="2">Cancel</Button>
					<Button Grid.Column="3" Command="{Binding ConfigureServerAsync}" Grid.RowSpan="2">Modify...</Button>
				</Grid>
			</Border>

			<StackPanel IsVisible="{Binding ToolCatalog.ShowTools}">
				<TextBlock Classes="SettingHeader">Updates</TextBlock>
				
				<Border Classes="Card" Padding="16 6 16 2">
					<StackPanel Spacing="10" Margin="0 0 0 5">
						<Grid ColumnDefinitions="*,Auto">
							<StackPanel Grid.Column="0" Margin="0 10 20 10">
								<TextBlock Classes="SettingCaption">Automatically update tools to latest version</TextBlock>
								<TextBlock Classes="SettingHelp">Installed tools will be updated automatically when new versions become available.</TextBlock>
							</StackPanel>
							<ToggleSwitch Grid.Column="1" VerticalAlignment="Center" OnContent="" OffContent="" IsChecked="{Binding ToolCatalog.AutoUpdate}"/>
						</Grid>
					</StackPanel>
				</Border>

				<TextBlock Classes="SettingHeader">Tools</TextBlock>

				<Border Classes="Card">
					<ListBox x:Name="ToolsList" BorderBrush="Black" x:DataType="local:GeneralSettingsViewModel" ItemsSource="{Binding ToolCatalog.Items}">
						<ListBox.ItemTemplate>
							<DataTemplate DataType="local:ToolCatalogItemViewModel">
								<Grid ColumnDefinitions="*,Auto,Auto,Auto,Auto,Auto,Auto">
									<Grid RowDefinitions="Auto,Auto" Margin="0 10">
										<TextBlock Text="{Binding Name}" VerticalAlignment="Center" FontSize="12" FontWeight="SemiBold" Grid.Row="0"/>
										<TextBlock Text="{Binding Description}" VerticalAlignment="Center" FontSize="12" FontWeight="Light" Grid.Row="1"/>
									</Grid>
									<ui:ProgressRing Width="16" Height="16" IsVisible="{Binding IsBusy}" VerticalAlignment="Center" Grid.Row="0" Grid.Column="1"/>
									<TextBlock Text="{Binding Status}" IsVisible="{Binding ShowStatusText}" VerticalAlignment="Center" Padding="5" FontSize="12" FontWeight="Light" Grid.Row="0" Grid.Column="2" Margin="4 0 4 0"/>
									<HyperlinkButton Content="{Binding Status}" IsVisible="{Binding ShowStatusLink}" Command="{Binding ShowLog}" FontSize="12" FontWeight="Light" Grid.Column="3" Margin="4 0 4 0"/>
									<Button Command="{Binding Add}" IsVisible="{Binding ShowAdd}" Grid.Column="4">Add</Button>
									<Button Command="{Binding Update}" IsVisible="{Binding ShowUpdate}" Grid.Column="5">Update</Button>
									<Button Command="{Binding Remove}" IsVisible="{Binding ShowRemove}" Grid.Column="6">Remove</Button>
								</Grid>
							</DataTemplate>
						</ListBox.ItemTemplate>
					</ListBox>
				</Border>
			</StackPanel>
			
		</StackPanel>
	</ScrollViewer>

</UserControl>
