// Copyright Epic Games, Inc. All Rights Reserved.

using Avalonia.Controls;

namespace UnrealToolbox
{
	partial class GeneralSettingsPage : UserControl, IDisposable
	{
		readonly GeneralSettingsViewModel _viewModel;

		public GeneralSettingsPage()
			: this(SettingsContext.Default)
		{
		}

		public GeneralSettingsPage(SettingsContext context)
		{
			InitializeComponent();

			_viewModel = new GeneralSettingsViewModel(context);

			DataContext = _viewModel;
		}

		public void Dispose()
		{
			_viewModel.Dispose();
		}
	}
}
