// Copyright Epic Games, Inc. All Rights Reserved.

using System.Diagnostics;
using Avalonia.Controls;
using Avalonia.Threading;
using CommunityToolkit.Mvvm.ComponentModel;
using EpicGames.Core;
using EpicGames.Horde;
using FluentAvalonia.UI.Controls;
using Microsoft.Extensions.DependencyInjection;

namespace UnrealToolbox
{
	partial class GeneralSettingsViewModel : ObservableObject, IDisposable
	{
		record class ConnectionState(string ServerUrl, string ServerStatus)
		{
			public static ConnectionState NoServerConfigured { get; } = new ConnectionState("No client configured", "Status unavailable");
		}

		readonly SettingsContext _context;
		readonly IHordeClientProvider _hordeClientProvider;
		readonly IToolCatalog? _toolCatalog;

		[ObservableProperty]
		string _serverUrl = String.Empty;

		[ObservableProperty]
		string _serverStatus = String.Empty;

		[ObservableProperty]
		bool _isConnecting;

		bool _requestRefresh;
		BackgroundTask<ConnectionState>? _connectTask;

		public ToolCatalogViewModel ToolCatalog { get; }

		public GeneralSettingsViewModel()
			: this(SettingsContext.Default)
		{
		}

		public GeneralSettingsViewModel(SettingsContext context)
		{
			_context = context;

			_hordeClientProvider = context.ServiceProvider.GetRequiredService<IHordeClientProvider>();
			_hordeClientProvider.OnStateChanged += StartRefresh;
//			_hordeClientProvider.OnAccessTokenStateChanged += StartRefresh;

			_toolCatalog = context.ServiceProvider.GetService<IToolCatalog>();
			ToolCatalog = new ToolCatalogViewModel(_toolCatalog);

			ServerUrl = HordeOptions.GetDefaultServerUrl()?.ToString() ?? "Unknown";
			ServerStatus = "Unknown";

			StartRefresh();
		}

		public async void Dispose()
		{
			_hordeClientProvider.OnStateChanged -= StartRefresh;
			//			_hordeClientProvider.OnAccessTokenStateChanged -= StartRefresh;

			if (_connectTask != null)
			{
				await _connectTask.DisposeAsync();
				_connectTask = null;
			}
		}

		void RunConnectionTask(Func<CancellationToken, Task<ConnectionState>> innerTask)
		{
			BackgroundTask<ConnectionState>? prevConnectTask = _connectTask;

			_connectTask = BackgroundTask.StartNew<ConnectionState>(async cancellationToken =>
			{
				if (prevConnectTask != null)
				{
					try
					{
						await prevConnectTask.DisposeAsync();
					}
					catch (OperationCanceledException)
					{
						// Ignore
					}
				}

				return await innerTask(cancellationToken);
			});
			_connectTask.Task?.ContinueWith(_ => Dispatcher.UIThread.Post(() => UpdateConnectionState()), TaskScheduler.Default);

			ServerStatus = "Connecting...";
			IsConnecting = true;
		}

		void UpdateConnectionState()
		{
			if (_connectTask != null && (_connectTask.Task?.IsCompleted ?? true))
			{
				ConnectionState? connectionState = null;
				if (_connectTask.Task != null)
				{
					try
					{
						_connectTask.Task.TryGetResult(out connectionState);
					}
					catch (Exception ex)
					{
						connectionState = new ConnectionState(ServerUrl, $"Unable to connect: {ex.Message}");
					}
				}

				if(connectionState != null)
				{
					ServerUrl = connectionState.ServerUrl;
					ServerStatus = connectionState.ServerStatus;
				}

				IsConnecting = false;

				_ = _connectTask.DisposeAsync().AsTask();
				_connectTask = null;
			}
		}

		public void OpenServer()
			=> OpenBrowser(ServerUrl);

		public void StartRefresh()
		{
			if (!_requestRefresh)
			{
				_requestRefresh = true;
				Dispatcher.UIThread.Post(() => UpdateRefresh());
			}
		}

		public void CancelRefresh()
			=> RunConnectionTask(_ => Task.FromResult<ConnectionState>(new ConnectionState(ServerUrl, "Cancelled")));

		void UpdateRefresh()
		{
			if (_requestRefresh)
			{
				_requestRefresh = false;
				RunConnectionTask(HandleRefreshAsync);
			}
		}

		async Task<ConnectionState> HandleRefreshAsync(CancellationToken cancellationToken)
		{
			using IHordeClientRef? hordeClientRef = _hordeClientProvider.GetClientRef();
			if (hordeClientRef == null)
			{
				return ConnectionState.NoServerConfigured;
			}

			IHordeClient client = hordeClientRef.Client;
			try
			{
				bool result = await client.LoginAsync(true, cancellationToken);
				if (!result)
				{
					ToolboxNotificationManager.PostNotification($"Connection Error: {client.ServerUrl.ToString()}", "Login Failed");
					return new ConnectionState(client.ServerUrl.ToString(), "Login failed");
				}
				else if (!client.HasValidAccessToken())
				{
					ToolboxNotificationManager.PostNotification($"Connection Error: {client.ServerUrl.ToString()}", "Session expired");
					return new ConnectionState(client.ServerUrl.ToString(), "Session expired");
				}
				else
				{
					_toolCatalog?.RequestUpdate();
					return new ConnectionState(client.ServerUrl.ToString(), "Authenticated");
				}
			}
			catch (Exception ex)
			{
				string message = ex.Message;
				message = message.Length > 100 ? message.Substring(0, 100) : message;

				// Make some connection errors more friendly
				string serverUrl = HordeOptions.GetDefaultServerUrl()?.ToString() ?? "Not Conigured";
				if (message.Contains("party did not properly respond", StringComparison.OrdinalIgnoreCase) || message.Contains("actively refused it", StringComparison.OrdinalIgnoreCase))
				{
					message = $"Unable to reach the Horde Server: {serverUrl}";
				}

				ToolboxNotificationManager.PostNotification($"Connection Error", $"{message}");

				return new ConnectionState(hordeClientRef.Client.ServerUrl.ToString(), $"Connection failed: {ex.Message}");
			}
		}
		
		public async Task ConfigureServerAsync()
		{
			string serverUrl = ServerUrl;

			TextBox serverUrlTextBox = new TextBox();
			serverUrlTextBox.Text = serverUrl;
			serverUrlTextBox.SelectionStart = serverUrl.Length;
			serverUrlTextBox.SelectionEnd = serverUrl.Length;

			ContentDialog dialog = new ContentDialog()
			{
				Title = "Connect to Server",
				Content = serverUrlTextBox,
				PrimaryButtonText = "Connect",
				CloseButtonText = "Cancel"
			};

			ContentDialogResult result = await dialog.ShowAsync(_context.SettingsWindow);
			if (result == ContentDialogResult.None)
			{
				return;
			}

			serverUrl = serverUrlTextBox.Text;

			try
			{
				Uri normalizedServerUrl = new Uri(serverUrl.Trim());
				HordeOptions.SetDefaultServerUrl(normalizedServerUrl);

				ServerUrl = normalizedServerUrl.ToString();
				ServerStatus = "Connecting...";

				_hordeClientProvider.Reset(); // Will trigger a call to StartRefresh()
			}
			catch (Exception ex)
			{
				ServerStatus = $"Error: {ex.Message}";
				RunConnectionTask(_ => Task.FromResult<ConnectionState>(new ConnectionState(ServerUrl, ServerStatus)));
			}
		}

		static void OpenBrowser(string url)
		{
			if (OperatingSystem.IsWindows())
			{
				string escapedUrl = url.Replace("&", "^&", StringComparison.Ordinal);
				Process.Start(new ProcessStartInfo("cmd", $"/c start {escapedUrl}") { CreateNoWindow = true });
			}
			else if (OperatingSystem.IsLinux())
			{
				Process.Start("xdg-open", url);
			}
			else if (OperatingSystem.IsMacOS())
			{
				Process.Start("open", url);
			}
		}
	}
}
