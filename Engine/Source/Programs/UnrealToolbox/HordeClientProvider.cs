// Copyright Epic Games, Inc. All Rights Reserved.

using System.Diagnostics;
using EpicGames.Core;
using EpicGames.Horde;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Logging;

namespace UnrealToolbox
{
	/// <summary>
	/// Implements a mechanism for creating scoped HordeClient instances, and modifying configuration settings
	/// </summary>
	class HordeClientProvider : IHordeClientProvider, IAsyncDisposable
	{
		class HordeClientLifetime : IAsyncDisposable
		{
			readonly TaskCompletionSource _refZeroTcs = new TaskCompletionSource(TaskCreationOptions.RunContinuationsAsynchronously);

			int _refCount;
			ServiceProvider _serviceProvider;
			IHordeClient _hordeClient;

			public IHordeClient Client => _hordeClient;

			public HordeClientLifetime(ILoggerFactory loggerFactory)
			{
				_refCount = 1;

				ServiceCollection serviceCollection = new ServiceCollection();
				serviceCollection.AddHorde(options => options.AllowAuthPrompt = false);
				serviceCollection.AddSingleton<ILoggerFactory>(loggerFactory);
				serviceCollection.AddSingleton(typeof(ILogger<>), typeof(Logger<>));
				_serviceProvider = serviceCollection.BuildServiceProvider();

				_hordeClient = _serviceProvider.GetRequiredService<IHordeClient>();
			}

			public async ValueTask DisposeAsync()
			{
				if (_serviceProvider != null)
				{
					Release();
					await _refZeroTcs.Task;

					await _serviceProvider.DisposeAsync();
					_serviceProvider = null!;
					_hordeClient = null!;
				}
			}

			public void AddRef()
			{
				int refCount = Interlocked.Increment(ref _refCount);
				Debug.Assert(refCount > 1);
			}

			public void Release()
			{
				if (Interlocked.Decrement(ref _refCount) == 0)
				{
					_refZeroTcs.SetResult();
				}
			}
		}

		class HordeClientRef : IHordeClientRef
		{
			HordeClientLifetime? _lifetime;

			public IHordeClient Client
				=> _lifetime?.Client ?? throw new ObjectDisposedException(null);

			public HordeClientRef(HordeClientLifetime lifetime)
			{
				_lifetime = lifetime;
				_lifetime.AddRef();
			}

			public void Dispose()
			{
				_lifetime?.Release();
				_lifetime = null;
			}
		}

		readonly object _lockObject = new object();
		readonly ILogger _logger;
		readonly ILoggerFactory _loggerFactory;
		readonly List<Task> _disposeTasks = new List<Task>();

		HordeClientLifetime? _lifetime;

		/// <summary>
		/// Event signalled whenever the connection state changes
		/// </summary>
		public event Action? OnStateChanged;

		/// <summary>
		/// Event signalled whenever the access token state changes
		/// </summary>
		public event Action? OnAccessTokenStateChanged;

		/// <summary>
		/// Constructor
		/// </summary>
		public HordeClientProvider(ILoggerFactory loggerFactory)
		{
			_logger = loggerFactory.CreateLogger<HordeClientProvider>();
			_loggerFactory = loggerFactory;

			CreateLifetime();
		}

		/// <inheritdoc/>
		public async ValueTask DisposeAsync()
		{
			if (_lifetime != null)
			{
				DestroyLifetime();
			}
			await Task.WhenAll(_disposeTasks);
		}

		/// <inheritdoc/>
		public IHordeClientRef? GetClientRef()
		{
			lock (_lockObject)
			{
				if (_lifetime == null)
				{
					return null;
				}
				else
				{
					return new HordeClientRef(_lifetime);
				}
			}
		}

		/// <inheritdoc/>
		public void Reset()
		{
			lock (_lockObject)
			{
				DestroyLifetime();
				CreateLifetime();
			}

			OnStateChanged?.Invoke();
		}

		void OnAccessTokenStateChangedInternal()
		{
			OnAccessTokenStateChanged?.Invoke();
		}

		void CreateLifetime()
		{
			Debug.Assert(_lifetime == null);

			try
			{
				_lifetime = new HordeClientLifetime(_loggerFactory);
				_lifetime.Client.OnAccessTokenStateChanged += OnAccessTokenStateChangedInternal;
			}
			catch (Exception ex)
			{
				_logger.LogError(ex, "Unable to create Horde client lifetime: {Message}", ex.Message);
				_lifetime = null;
			}
		}

		void DestroyLifetime()
		{
			if (_lifetime != null)
			{
				_lifetime.Client.OnAccessTokenStateChanged -= OnAccessTokenStateChangedInternal;
				_disposeTasks.Add(_lifetime.DisposeAsync().AsTask());
				_lifetime = null;
			}

			AsyncUtils.RemoveCompleteTasks(_disposeTasks);
		}
	}
}
