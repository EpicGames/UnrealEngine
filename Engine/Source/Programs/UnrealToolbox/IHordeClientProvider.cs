// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde;

namespace UnrealToolbox
{
	/// <summary>
	/// Reference to a Horde client instance. Should be disposed of when done.
	/// </summary>
	interface IHordeClientRef : IDisposable
	{
		/// <summary>
		/// The client instance
		/// </summary>
		IHordeClient Client { get; }
	}

	/// <summary>
	/// Accessor for a <see cref="IHordeClient"/> instance which can be recreated in response to config changes. 
	/// </summary>
	interface IHordeClientProvider
	{
		/// <summary>
		/// Event signalled whenever the connection state changes
		/// </summary>
		event Action? OnStateChanged;

		/// <summary>
		/// Event signalled whenever the access token state changes
		/// </summary>
		event Action? OnAccessTokenStateChanged;

		/// <summary>
		/// Resets the current client and creates a new one with the latest settings.
		/// </summary>
		void Reset();

		/// <summary>
		/// Gets a reference to the current client instance. These references should be kept as short as possible.
		/// </summary>
		IHordeClientRef? GetClientRef();
	}
}
