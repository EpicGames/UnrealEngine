// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealToolbox
{
	/// <summary>
	/// Interface for the tool catalog
	/// </summary>
	public interface IToolCatalog
	{
		/// <summary>
		/// Whether to auto-update tools to latest
		/// </summary>
		bool AutoUpdate { get; set; }
		
		/// <summary>
		/// Asynchronously request an update of tool catalog
		/// Listen on <see cref="OnItemsChanged"/> for when items actually update.
		/// </summary>
		void RequestUpdate();

		/// <summary>
		/// List of available tools
		/// </summary>
		IReadOnlyList<IToolCatalogItem> Items { get; }

		/// <summary>
		/// Notification that some property of the items list has changed
		/// </summary>
		event Action? OnItemsChanged;
	}
}
