// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using EpicGames.Horde.Tools;

namespace UnrealToolbox
{
	/// <summary>
	/// Describes a tool available to be installed
	/// </summary>
	public interface IToolCatalogItem
	{
		/// <summary>
		/// Identifier for the tool
		/// </summary>
		ToolId Id { get; }

		/// <summary>
		/// Name of the tool
		/// </summary>
		string Name { get; }

		/// <summary>
		/// Description for the tool
		/// </summary>
		string Description { get; }

		/// <summary>
		/// Latest available version of the tool
		/// </summary>
		ToolDeploymentInfo? Latest { get; }

		/// <summary>
		/// The current tool deployment
		/// </summary>
		CurrentToolDeploymentInfo? Current { get; }

		/// <summary>
		/// Pending tool deployment, if an install or uninstall operation 
		/// </summary>
		PendingToolDeploymentInfo? Pending { get; }

		/// <summary>
		/// Callback for the item state changing
		/// </summary>
		event Action? OnItemChanged;

		/// <summary>
		/// Cancel any pending installation
		/// </summary>
		void Cancel();

		/// <summary>
		/// Start installing this tool or updating it to the latest version
		/// </summary>
		void Install();

		/// <summary>
		/// Start uninstalling this tool
		/// </summary>
		void Uninstall();
	}

	/// <summary>
	/// Information about a particular tool deployment
	/// </summary>
	/// <param name="Id">Identifier for the deployment</param>
	/// <param name="Version">Version number for this deployment</param>
	public record class ToolDeploymentInfo(ToolDeploymentId Id, string Version);

	/// <summary>
	/// Information about an installed tool deployment
	/// </summary>
	/// <param name="Id">Identifier for the deployment</param>
	/// <param name="Version">Version number for this deployment</param>
	/// <param name="Dir">Directory containing the tool</param>
	/// <param name="Config">Configuration for the tool</param>
	public record class CurrentToolDeploymentInfo(ToolDeploymentId Id, string Version, DirectoryReference Dir, ToolConfig Config) : ToolDeploymentInfo(Id, Version);

	/// <summary>
	/// Information about a pending tool installation
	/// </summary>
	/// <param name="Failed">Whether the installation failed</param>
	/// <param name="Message">Current status message</param>
	/// <param name="Deployment">Information about the pending deployment</param>
	/// <param name="ShowLogLink">Whether to show a link to the log</param>
	public record class PendingToolDeploymentInfo(bool Failed, string? Message, ToolDeploymentInfo? Deployment, bool ShowLogLink = false);
}
