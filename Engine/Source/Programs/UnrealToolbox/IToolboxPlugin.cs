// Copyright Epic Games, Inc. All Rights Reserved.

using Avalonia.Controls;
using FluentAvalonia.UI.Controls;

namespace UnrealToolbox
{
	enum TrayAppPluginState
	{
		Undefined,
		Ok,
		Busy,
		Paused,
		Error
	}

	/// <summary>
	/// Reported status of a plugin
	/// </summary>
	record class ToolboxPluginStatus(TrayAppPluginState State, string? Message = null)
	{
		public static ToolboxPluginStatus Default { get; } = new ToolboxPluginStatus(TrayAppPluginState.Undefined);
	}

	/// <summary>
	/// Plugin for the tray app
	/// </summary>
	interface IToolboxPlugin : IAsyncDisposable
	{
		/// <summary>
		/// Name of the plugin to show in the settings page
		/// </summary>
		string Name { get; }

		/// <summary>
		/// Icon to show on the settings page
		/// </summary>
		IconSource Icon { get; }

		/// <summary>
		/// Url protocol handlers that this plugin supports
		/// </summary>
		IReadOnlyList<string> UrlProtocols { get; }

		/// <summary>
		/// Refresh the state of this plugin. Called when the application is activated or focussed.
		/// </summary>
		/// <returns>True if the plugin status has changed, and the context menu needs to be rebuilt</returns>
		bool Refresh();

		/// <summary>
		/// Handles a url passed on the commandline
		/// </summary>
		bool HandleUrl(Uri url);

		/// <summary>
		/// Get the current status of this plugin
		/// </summary>
		ToolboxPluginStatus GetStatus();

		/// <summary>
		/// Determines if the plugin should be shown in the settings page
		/// </summary>
		bool HasSettingsPage();

		/// <summary>
		/// Create a settings page for this plugin
		/// </summary>
		Control CreateSettingsPage(SettingsContext context);

		/// <summary>
		/// Allow the plugin to customize the tray icon context menu
		/// </summary>
		void PopulateContextMenu(NativeMenu contextMenu);
	}

	/// <summary>
	/// Base implementation of <see cref="IToolboxPlugin"/>
	/// </summary>
	abstract class ToolboxPluginBase : IToolboxPlugin
	{
		/// <inheritdoc/>
		public abstract string Name { get; }

		/// <inheritdoc/>
		public virtual IconSource Icon
			=> new SymbolIconSource() { Symbol = Symbol.Help };

		/// <inheritdoc/>
		public virtual IReadOnlyList<string> UrlProtocols
			=> Array.Empty<string>();

		/// <inheritdoc/>
		public virtual bool Refresh()
			=> false;

		/// <inheritdoc/>
		public virtual bool HandleUrl(Uri url)
			=> false;

		/// <inheritdoc/>
		public virtual Control CreateSettingsPage(SettingsContext context)
			=> throw new NotSupportedException();

		/// <inheritdoc/>
		public virtual ValueTask DisposeAsync()
		{
			GC.SuppressFinalize(this);
			return default;
		}

		/// <inheritdoc/>
		public virtual ToolboxPluginStatus GetStatus()
			=> ToolboxPluginStatus.Default;

		/// <inheritdoc/>
		public virtual bool HasSettingsPage()
			=> false;

		/// <inheritdoc/>
		public virtual void PopulateContextMenu(NativeMenu contextMenu)
		{ }
	}
}
