// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealToolbox
{
	/// <summary>
	/// Interface for the tray app host application
	/// </summary>
	interface IToolboxPluginHost
	{
		/// <summary>
		/// Notifies the host that a status change has ocurred
		/// </summary>
		void UpdateStatus();
	}
}
