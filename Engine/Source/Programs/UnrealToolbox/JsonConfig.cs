// Copyright Epic Games, Inc. All Rights Reserved.

using System.Text.Json;
using System.Text.Json.Serialization;
using EpicGames.Core;

namespace UnrealToolbox
{
	class JsonConfig<T> where T : class, new()
	{
		readonly FileReference _file;
		static readonly JsonSerializerOptions s_jsonSerializerOptions = GetJsonSerializerOptions();
		T _current;
		byte[] _data;

		public T Current => _current;

		public JsonConfig(FileReference file)
		{
			_file = file;
			_current = new T();
			_data = Array.Empty<byte>();
		}

		static JsonSerializerOptions GetJsonSerializerOptions()
		{
			JsonSerializerOptions options = new JsonSerializerOptions();
			options.PropertyNamingPolicy = JsonNamingPolicy.CamelCase;
			options.PropertyNameCaseInsensitive = true;
			options.DefaultIgnoreCondition = JsonIgnoreCondition.WhenWritingNull;
			options.AllowTrailingCommas = true;
			options.WriteIndented = true;
			options.Converters.Add(new JsonStringEnumConverter());
			return options;
		}

		public bool LoadSettings()
		{
			if (FileReference.Exists(_file))
			{
				try
				{
					byte[] data = FileReference.ReadAllBytes(_file);
					if (!data.SequenceEqual(_data))
					{
						_current = JsonSerializer.Deserialize<T>(data, s_jsonSerializerOptions)!;
						_data = data;
						return true;
					}
				}
				catch (Exception)
				{
				}
			}
			return false;
		}

		public bool UpdateSettings(T settings)
		{
			byte[] data = JsonSerializer.SerializeToUtf8Bytes(settings, s_jsonSerializerOptions);
			if (!data.SequenceEqual(_data))
			{
				_current = settings;
				_data = data;

				DirectoryReference.CreateDirectory(_file.Directory);
				FileReference.WriteAllBytes(_file, data);

				return true;
			}
			return false;
		}
	}
}
