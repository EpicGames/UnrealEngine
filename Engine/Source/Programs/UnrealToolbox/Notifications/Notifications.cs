// Copyright Epic Games, Inc. All Rights Reserved.

using System.Reflection;
using System.Runtime.InteropServices;
using DesktopNotifications;
using DesktopNotifications.Windows;
using EpicGames.Core;

namespace UnrealToolbox
{
	/// <summary>
	/// A tool notification
	/// </summary>
	class ToolboxNotification
	{
		/// <summary>
		/// The title to display
		/// </summary>
		public string Title { get; }

		/// <summary>
		/// The body of the notification
		/// </summary>
		public string Body { get; }

		/// <summary>
		/// Whether to force the notification regardless of delta time
		/// </summary>
		public bool Force { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="title"></param>
		/// <param name="body"></param>
		/// <param name="force"></param>
		public ToolboxNotification(string title, string body, bool force = false)
		{
			Title = title;
			Body = body;
			Force = force;
		}		
	}

	/// <summary>
	/// Toolbox notification manager implementation
	/// </summary>
	class ToolboxNotificationManager : IAsyncDisposable
	{
		private INotificationManager? _platformManager;

		private static readonly object s_lock = new object();

		private static List<ToolboxNotification> s_notifications = new List<ToolboxNotification>();
		readonly BackgroundTask _backgroundTask;

		// spam prevention
		DateTime? _lastNotificationTime;
		string? _lastTitle;
		string? _lastBody;

		[DllImport("shell32.dll", SetLastError = true)]
		private static extern void SetCurrentProcessExplicitAppUserModelID([MarshalAs(UnmanagedType.LPWStr)] string appId);

		public ToolboxNotificationManager()
		{
			_backgroundTask = new BackgroundTask(PostNotificationsAsync);
		}

		async Task PostNotificationsAsync(CancellationToken cancellationToken)
		{
			for (; ; )
			{
				try
				{
					List<ToolboxNotification>? notifications = null;
					lock (s_lock)
					{
						if (s_notifications.Count > 0)
						{
							notifications = s_notifications;
							s_notifications = new List<ToolboxNotification>();
						}	
					}

					if (notifications != null && notifications.Count > 0)
					{
						notifications.Reverse();
						ShowNotification(notifications[0].Title, notifications[0].Body, notifications[0].Force);
					}
				}
				catch (OperationCanceledException)
				{
					throw;
				}
				catch (Exception)
				{
					
				}

				await Task.Delay(TimeSpan.FromSeconds(1.0), cancellationToken);
			}
		}

		public void Start()
		{
			if (Environment.OSVersion.Platform == PlatformID.Win32NT)
			{
				// WindowsApplicationContext.FromCurrentProcess() has side effects of creating start menu items, and changing the app user model id to the executing assembly, which can be dotnet.exe
				// WindowsApplicationContext context = WindowsApplicationContext.FromCurrentProcess();
				WindowsApplicationContext? context = Activator.CreateInstance(type: typeof(WindowsApplicationContext), bindingAttr: BindingFlags.Instance | BindingFlags.NonPublic, binder: null, args: new object[] { "Unreal Toolbox", "Unreal Toolbox" }, culture: null) as WindowsApplicationContext;
				SetCurrentProcessExplicitAppUserModelID("Unreal Toolbox");

				_platformManager = new WindowsNotificationManager(context);
			}
			else
			{
				throw new NotImplementedException();
			}

			// initialize and ensure with result
			_platformManager.Initialize().GetAwaiter().GetResult();

			_backgroundTask.Start();

		}

		public async ValueTask DisposeAsync()
		{
			_platformManager?.Dispose();
			await _backgroundTask.DisposeAsync();
		}

		/// <summary>
		/// Threead safe notification posting
		/// </summary>
		/// <param name="title"></param>
		/// <param name="body"></param>
		/// <param name="force"></param>
		public static void PostNotification(string title, string body, bool force = false)
		{
			lock (s_lock)
			{
				s_notifications.Add(new ToolboxNotification(title, body, force));
			}
		}

		/// <summary>
		/// Show a notification, 
		/// </summary>
		/// <param name="title"></param>
		/// <param name="body"></param>
		/// <param name="force"></param>
		private void ShowNotification(string title, string body, bool force = false)
		{
			if (_platformManager == null)
			{
				return;
			}

			// spawn 
			if (!force && _lastNotificationTime != null && !String.IsNullOrEmpty(_lastBody) && !String.IsNullOrEmpty(_lastTitle))
			{
				TimeSpan deltaTime = DateTime.Now - _lastNotificationTime.Value;

				// don't show a new notification if already displayed one in last 2 minutes
				if (deltaTime.TotalSeconds < 120)
				{
					return;
				}

				// if the title and body are the same, wait 10 minutes
				if ((deltaTime.TotalSeconds < 600) && title == _lastTitle && body == _lastBody)
				{
					return;
				}
			}

			_lastTitle = title;
			_lastBody = body;
			_lastNotificationTime = DateTime.Now;

			Notification notification = new Notification
			{
				Title = title,
				Body = body
			};

			_platformManager.ShowNotification(notification, DateTimeOffset.Now + TimeSpan.FromSeconds(30));
		}
	}
}