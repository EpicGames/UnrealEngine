// Copyright Epic Games, Inc. All Rights Reserved.

using System.Runtime.Versioning;
using Avalonia.Controls;
using EpicGames.Core;
using EpicGames.Horde;
using EpicGames.Horde.Storage;
using EpicGames.Horde.Storage.Nodes;
using FluentAvalonia.UI.Controls;
using Microsoft.Extensions.Logging;
using Microsoft.Win32;

namespace UnrealToolbox.Plugins.Artifacts
{
	class ArtifactsPlugin : ToolboxPluginBase
	{
		readonly IHordeClientProvider _hordeClientProvider;
		readonly JsonConfig<DownloadSettings> _settings;
		readonly ILogger _logger;

		public override string Name
			=> "Artifact Downloader";

		public override IconSource Icon
			=> new SymbolIconSource() { Symbol = Symbol.Download };

		public override IReadOnlyList<string> UrlProtocols
			=> new[] { "horde-artifact" };

		public ArtifactsPlugin(IHordeClientProvider hordeClientProvider, ILogger<ArtifactsPlugin> logger)
		{
			_hordeClientProvider = hordeClientProvider;
			_settings = new JsonConfig<DownloadSettings>(FileReference.Combine(Program.DataDir, "Artifacts.json"));
			_settings.LoadSettings();
			_logger = logger;
		}

		public override bool HandleUrl(Uri url)
		{
			if (!url.Scheme.Equals("horde-artifact", StringComparison.OrdinalIgnoreCase))
			{
				return false;
			}

			_ = DownloadAsync(url, CancellationToken.None);
			return true;
		}

		public override void PopulateContextMenu(NativeMenu contextMenu)
		{
			NativeMenuItem menuItem = new NativeMenuItem("Open artifact...");
			menuItem.Click += MenuItem_Click;
			contextMenu.Add(menuItem);
		}

		private void MenuItem_Click(object? sender, EventArgs e)
		{
			_ = DownloadAsync(null, CancellationToken.None);
		}

		public async Task DownloadAsync(Uri? artifactUrl, CancellationToken cancellationToken)
		{
			DownloadOptionsWindow window = new DownloadOptionsWindow(artifactUrl, _settings, _hordeClientProvider);
			window.Show();

			Program.BringWindowToFront(window);

			DownloadOptions? options = await window.Result;
			if (options != null)
			{
				await DownloadDataAsync(options, cancellationToken);
			}
		}

		public async Task DownloadDataAsync(DownloadOptions options, CancellationToken cancellationToken)
		{
			DownloadProgressViewModel progress = new DownloadProgressViewModel();

			using CancellationTokenSource cancellationTokenSource = CancellationTokenSource.CreateLinkedTokenSource(cancellationToken);

			EventHandler? closedEventHandler = (_, _) => cancellationTokenSource.Cancel();

			DownloadProgressWindow progressWindow = new DownloadProgressWindow(progress);
			progressWindow.Closed += closedEventHandler;
			progressWindow.Show();

			try
			{
				CancellationToken linkedCancellationToken = cancellationTokenSource.Token;

				Uri serverUrl = new Uri(options.BaseUrl.GetLeftPart(UriPartial.Authority));

				using IHordeClientRef? hordeClientRef = _hordeClientProvider.GetClientRef();
				if (hordeClientRef != null)
				{
					IHordeClient hordeClient = hordeClientRef.Client;
					IStorageNamespace storageNamespace = hordeClient.GetStorageNamespace(options.BaseUrl.AbsolutePath);

					progress.Status = "Connecting to server...";

					IBlobRef<DirectoryNode>? blobRef = await storageNamespace.ReadRefAsync<DirectoryNode>(options.RefName, cancellationToken: linkedCancellationToken);

					if (options.PatchExistingData)
					{
						Workspace workspace = await Workspace.CreateOrOpenAsync(options.OutputDir, _logger, cancellationToken);

						progress.Status = "Reconciling current data...";
						await workspace.ReconcileAsync(cancellationToken);

						DirectoryNode node = await blobRef.ReadBlobAsync(cancellationToken: linkedCancellationToken);
						progress.SetTotalSize(node.Length);

						progress.Status = "Starting...";
						await workspace.SyncAsync(WorkspaceLayerId.Default, blobRef, new ExtractOptions { Progress = progress, ProgressUpdateFrequency = TimeSpan.FromSeconds(0.2) }, cancellationToken: cancellationToken);
					}
					else
					{
						progress.Status = "Starting...";
						await blobRef.ExtractAsync(options.OutputDir.ToDirectoryInfo(), new ExtractOptions { Progress = progress, ProgressUpdateFrequency = TimeSpan.FromSeconds(0.2) }, _logger, linkedCancellationToken);
					}

					progress.Status = "Download complete.";
					progress.MarkComplete();
				}

				progress.OpenFolderPath = options.OutputDir.ToString();
			}
			catch (OperationCanceledException)
			{
				progressWindow.Close();
			}
			catch (Exception ex)
			{
				progress.ErrorMessage = $"Error while downloading artifact: {ex.ToString()}";
			}
			finally
			{
				progress.IsRunning = false;
				progressWindow.Closed -= closedEventHandler;
			}
		}

		[SupportedOSPlatform("windows")]
		public static void RegisterFileAssociations(string application)
		{
			string progId = "UnrealGameSync.Artifact";

			using (RegistryKey baseKey = Registry.CurrentUser.CreateSubKey("SOFTWARE\\Classes"))
			{
				using (RegistryKey extensionKey = baseKey.CreateSubKey(".uartifact"))
				{
					extensionKey.SetValue("", progId);
				}

				using (RegistryKey progKey = baseKey.CreateSubKey(progId))
				{
					progKey.SetValue("", "Horde artifact descriptor");

					using (RegistryKey defaultIconKey = progKey.CreateSubKey("DefaultIcon"))
					{
						defaultIconKey.SetValue("", $"\"{application}\",0");
					}

					using (RegistryKey shellKey = progKey.CreateSubKey("Shell"))
					{
						using (RegistryKey openKey = shellKey.CreateSubKey("open"))
						{
							using RegistryKey commandKey = openKey.CreateSubKey("command");
							commandKey.SetValue("", $"\"{application}\" -artifact=\"%1\"");
						}
					}
				}
			}
		}
	}
}
