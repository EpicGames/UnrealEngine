// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using EpicGames.Horde.Storage;

namespace UnrealToolbox.Plugins.Artifacts
{
	class DownloadOptions
	{
		public Uri BaseUrl { get; set; }
		public RefName RefName { get; set; }
		public DirectoryReference OutputDir { get; set; }
		public bool PatchExistingData { get; set; }

		public DownloadOptions(Uri baseUrl, RefName refName, DirectoryReference outputDir, bool patchExistingData)
		{
			BaseUrl = baseUrl;
			RefName = refName;
			OutputDir = outputDir;
			PatchExistingData = patchExistingData;
		}
	}
}
