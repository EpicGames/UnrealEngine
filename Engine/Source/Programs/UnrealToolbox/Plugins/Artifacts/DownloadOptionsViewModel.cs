// Copyright Epic Games, Inc. All Rights Reserved.

using System.Diagnostics;
using System.Text.RegularExpressions;
using Avalonia.Controls;
using Avalonia.Platform.Storage;
using CommunityToolkit.Mvvm.ComponentModel;
using EpicGames.Core;
using EpicGames.Horde;
using EpicGames.Horde.Artifacts;
using EpicGames.Horde.Commits;
using EpicGames.Horde.Storage;
using EpicGames.Horde.Streams;
using FluentAvalonia.UI.Controls;

namespace UnrealToolbox.Plugins.Artifacts
{
	record class ArtifactInfo(ArtifactId? Id, string? Name, string? Type, StreamId? Stream, CommitId? Commit, string? Description, Uri? JobUrl, IReadOnlyList<string>? Keys, IReadOnlyList<string>? Metadata, Uri BaseUrl, RefName RefName);

	partial class DownloadOptionsViewModel : ObservableObject
	{
		readonly Window _window;

		[ObservableProperty]
		[NotifyPropertyChangedFor(nameof(InputText))]
		Uri? _artifactUrl;

		[ObservableProperty]
		bool _showUrlSpinner;

		[ObservableProperty]
		[NotifyPropertyChangedFor(nameof(AllowStart))]
		[NotifyPropertyChangedFor(nameof(BuildName))]
		[NotifyPropertyChangedFor(nameof(ShowArtifactInfo))]
		[NotifyPropertyChangedFor(nameof(HasArtifactName))]
		[NotifyPropertyChangedFor(nameof(ArtifactName))]
		[NotifyPropertyChangedFor(nameof(HasArtifactType))]
		[NotifyPropertyChangedFor(nameof(ArtifactType))]
		[NotifyPropertyChangedFor(nameof(HasArtifactDescription))]
		[NotifyPropertyChangedFor(nameof(ArtifactDescription))]
		[NotifyPropertyChangedFor(nameof(HasArtifactJobUrl))]
		[NotifyPropertyChangedFor(nameof(ArtifactJobUrl))]
		[NotifyPropertyChangedFor(nameof(ArtifactKeys))]
		[NotifyPropertyChangedFor(nameof(ArtifactMetadata))]
		[NotifyPropertyChangedFor(nameof(OutputDir))]
		ArtifactInfo? _artifact;

		public string InputText
		{
			get
			{
				if (ArtifactUrl == null)
				{
					return "No file selected.";
				}
				else if (ArtifactUrl.IsFile)
				{
					return ArtifactUrl.LocalPath;
				}
				else
				{
					return ArtifactUrl.ToString();
				}
			}
		}

		public bool ShowBrowse
			=> ArtifactUrl == null || ArtifactUrl.IsFile;

		public string OutputDir
			=> AppendBuildName ? Path.Combine(BaseOutputDir, BuildName) : BaseOutputDir;

		[ObservableProperty]
		[NotifyPropertyChangedFor(nameof(OutputDir))]
		public string _baseOutputDir;

		[ObservableProperty]
		[NotifyPropertyChangedFor(nameof(OutputDir))]
		public bool _appendBuildName;

		[ObservableProperty]
		public bool _patchExistingData;

		[ObservableProperty]
		[NotifyPropertyChangedFor(nameof(ShowErrorMessage))]
		string _errorMessage = String.Empty;

		public bool ShowArtifactInfo
			=> Artifact != null;

		public bool HasArtifactName => !String.IsNullOrEmpty(ArtifactName);

		public string ArtifactName
			=> Artifact?.Name ?? String.Empty;

		public bool HasArtifactType => !String.IsNullOrEmpty(Artifact?.Type);

		public string ArtifactType
			=> Artifact?.Type ?? String.Empty;

		public bool HasArtifactDescription
			=> !String.IsNullOrEmpty(ArtifactDescription);

		public string ArtifactDescription
			=> Artifact?.Description ?? String.Empty;

		public bool HasArtifactJobUrl
			=> !String.IsNullOrEmpty(ArtifactJobUrl);

		public string ArtifactJobUrl
			=> Artifact?.JobUrl?.ToString() ?? String.Empty;

		public string ArtifactKeys
		{
			get
			{
				List<string> keysList = (Artifact?.Keys ?? new List<string>()).Where(x => !x.StartsWith("job:", StringComparison.OrdinalIgnoreCase)).ToList();
				return (keysList.Count == 0) ? "(None)" : String.Join(", ", keysList);
			}
		}

		public string ArtifactMetadata
			=> (Artifact?.Metadata == null) ? "(None)" : String.Join(", ", Artifact.Metadata);

		public bool ShowErrorMessage
			=> ErrorMessage.Length > 0;

		public bool AllowStart
			=> Artifact != null;

		public string BuildName
		{
			get
			{
				if (Artifact != null && Artifact.Stream != null && Artifact.Commit != null)
				{
					return $"{Artifact.Stream}-{Artifact.Commit}";
				}
				else
				{
					return "";
				}
			}
		}

		readonly IHordeClientProvider? _hordeClientProvider;

		public DownloadOptionsViewModel(Window window, DownloadSettings? settings, IHordeClientProvider? hordeClientProvider)
		{
			_window = window;
			_baseOutputDir = settings?.OutputDir ?? 
				GetDefaultDownloadFolder(window) ??
				Environment.GetFolderPath(Environment.SpecialFolder.MyDocuments) ??
				Directory.GetCurrentDirectory();
			_appendBuildName = settings?.AppendBuildName ?? true;
			_patchExistingData = settings?.PatchExistingData ?? false;
			_hordeClientProvider = hordeClientProvider;
		}

		static string? GetDefaultDownloadFolder(Window window)
		{
			IStorageFolder? storageFolder = Task.Run(async () => await window.StorageProvider.TryGetWellKnownFolderAsync(WellKnownFolder.Downloads)).Result;
			return storageFolder?.TryGetLocalPath();
		}

		public DownloadOptions? GetDownloadOptions()
		{
			if (Artifact == null)
			{
				return null;
			}
			if (String.IsNullOrEmpty(OutputDir))
			{
				return null;
			}

			return new DownloadOptions(Artifact.BaseUrl, Artifact.RefName, new DirectoryReference(OutputDir), PatchExistingData);
		}

		public async Task<bool> RefreshArtifactAsync()
		{
			ShowUrlSpinner = true;
			try
			{
				Uri? artifactUrl = ArtifactUrl;
				if (artifactUrl == null)
				{
					Artifact = null;

					ErrorMessage = String.Empty;
					return false;
				}
				else if (artifactUrl.IsFile)
				{
					FileReference file = new FileReference(artifactUrl.LocalPath);
					if (!FileReference.Exists(file))
					{
						ErrorMessage = "Artifact descriptor not found.";
						return false;
					}

					ArtifactDescriptor descriptor = await ArtifactDescriptor.ReadAsync(file, CancellationToken.None);
					Artifact = new ArtifactInfo(null, descriptor.Name, descriptor.Type, null, null, descriptor.Description, descriptor.JobUrl, descriptor.Keys, descriptor.Metadata, descriptor.BaseUrl, descriptor.RefName);

					ErrorMessage = String.Empty;
					return true;
				}
				else
				{
					using IHordeClientRef? clientRef = _hordeClientProvider!.GetClientRef();
					if (clientRef == null)
					{
						ErrorMessage = $"No server is configured.";
						return false;
					}

					using HordeHttpClient httpClient = clientRef.Client.CreateHttpClient();

					if (artifactUrl.Host != httpClient.BaseUrl.Host)
					{
						ErrorMessage = $"Given url targets server {artifactUrl.Host} rather than configured server {httpClient.BaseUrl.Host}";
						return false;
					}

					GetArtifactResponse response = await HordeHttpRequest.GetAsync<GetArtifactResponse>(httpClient.HttpClient, artifactUrl.PathAndQuery, CancellationToken.None);

					Uri? jobUrl = null;
					foreach (string key in response.Keys)
					{
						Match match = Regex.Match(key, "^job:([0-9a-fA-F]+)$");
						if (match.Success)
						{
							jobUrl = new Uri(httpClient.BaseUrl, $"job/{match.Groups[1].Value}");
							break;
						}
					}

					Uri baseUrl = new Uri(httpClient.BaseUrl, artifactUrl.PathAndQuery);
					Artifact = new ArtifactInfo(response.Id, response.Name.ToString(), response.Type.ToString(), response.StreamId, response.CommitId, response.Description, jobUrl, response.Keys, response.Metadata, baseUrl, "default");

					ErrorMessage = String.Empty;
					return true;
				}
			}
			catch (Exception ex)
			{
				ErrorMessage = ex.Message;
				return false;
			}
			finally
			{
				ShowUrlSpinner = false;
			}
		}

		public async Task<bool> SelectUrlAsync()
		{
			string text = ArtifactUrl?.ToString() ?? String.Empty;

			TextBox urlTextBox = new TextBox();
			urlTextBox.Text = text;
			urlTextBox.SelectionStart = text.Length;
			urlTextBox.SelectionEnd = text.Length;

			ContentDialog dialog = new ContentDialog()
			{
				Title = "Open URL",
				Content = urlTextBox,
				PrimaryButtonText = "Connect",
				CloseButtonText = "Cancel"
			};

			ContentDialogResult result = await dialog.ShowAsync(_window);
			if (result == ContentDialogResult.None || urlTextBox.Text == null)
			{
				return false;
			}

			ArtifactUrl = new Uri(urlTextBox.Text);
			return await RefreshArtifactAsync();
		}

		public async Task<bool> SelectInputFileAsync()
		{
			FilePickerOpenOptions options = new FilePickerOpenOptions();
			options.AllowMultiple = false;
			options.FileTypeFilter = [new FilePickerFileType("Artifacts") { Patterns = ["*.uartifact"] }];

			IReadOnlyList<IStorageFile> files = await _window.StorageProvider.OpenFilePickerAsync(options);
			if (files.Count == 0)
			{
				return false;
			}

			string? localPath = files[0].TryGetLocalPath();
			if (localPath == null)
			{
				return false;
			}

			ArtifactUrl = new Uri(new Uri("file://"), localPath);
			return await RefreshArtifactAsync();
		}

		public void OpenJobUrl()
		{
			if (!String.IsNullOrEmpty(ArtifactJobUrl))
			{
				string url = ArtifactJobUrl;
				if (OperatingSystem.IsWindows())
				{
					Process.Start(new ProcessStartInfo(url) { UseShellExecute = true })?.Dispose();
				}
				else if (OperatingSystem.IsLinux())
				{
					Process.Start("xdg-open", url)?.Dispose();
				}
				else if (OperatingSystem.IsMacOS())
				{
					Process.Start("open", url)?.Dispose();
				}
				else
				{
					throw new NotImplementedException();
				}
			}
		}

		public void OpenOutputDir()
		{
			try
			{
				Process.Start(new ProcessStartInfo { FileName = OutputDir, UseShellExecute = true });
			}
			catch (Exception ex)
			{
				ErrorMessage = ex.Message;
			}
		}

		public async Task<bool> SelectOutputDirAsync()
		{
			FolderPickerOpenOptions options = new FolderPickerOpenOptions();
			options.AllowMultiple = false;
			options.SuggestedStartLocation = await _window.StorageProvider.TryGetFolderFromPathAsync(OutputDir);

			IReadOnlyList<IStorageFolder> folders = await _window.StorageProvider.OpenFolderPickerAsync(options);
			if (folders.Count == 0)
			{
				return false;
			}

			string? localPath = folders[0].TryGetLocalPath();
			if (localPath == null)
			{
				return false;
			}

			BaseOutputDir = localPath;
			return true;
		}
	}
}
