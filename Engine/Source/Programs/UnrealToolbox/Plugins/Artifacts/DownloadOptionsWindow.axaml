<Window xmlns="https://github.com/avaloniaui"
        xmlns:x="http://schemas.microsoft.com/winfx/2006/xaml"
        xmlns:d="http://schemas.microsoft.com/expression/blend/2008"
        xmlns:mc="http://schemas.openxmlformats.org/markup-compatibility/2006"
		xmlns:ui="using:FluentAvalonia.UI.Controls"
		xmlns:local="using:UnrealToolbox.Plugins.Artifacts"
        mc:Ignorable="d" d:DesignWidth="800" 
		x:Class="UnrealToolbox.Plugins.Artifacts.DownloadOptionsWindow"
		x:DataType="local:DownloadOptionsViewModel"
		Width="800"
		SizeToContent="Height"
        Title="Download Artifact"
		ExtendClientAreaToDecorationsHint="False"
		WindowStartupLocation="CenterScreen">

	<Window.Styles>
		<StyleInclude Source="/Styles.axaml"/>
	</Window.Styles>

	<StackPanel Margin="20 0 20 20" Spacing="5">

		<TextBlock Classes="SettingHeader">Source</TextBlock>

		<Border Classes="Card">
			<StackPanel Spacing="5" Margin="0">
				<Grid ColumnDefinitions="Auto,*,Auto" RowDefinitions="Auto">
					<ui:ProgressRing Width="16" Height="16" IsVisible="{Binding ShowUrlSpinner}" VerticalAlignment="Center" Grid.Row="0" Grid.Column="0" Margin="0 0 8 0"/>
					<SelectableTextBlock Text="{Binding InputText}" VerticalAlignment="Center" Grid.Column="1"/>
					<Button Content="Browse..." Margin="16 0 0 0" Width="120" Grid.Column="2" Command="{Binding SelectInputFileAsync}" IsVisible="{Binding ShowBrowse}"/>
				</Grid>
			</StackPanel>
		</Border>
		<Border Classes="Card" BorderThickness="1 0 1 1" IsVisible="{Binding ShowArtifactInfo}">
			<Grid ColumnDefinitions="Auto,*" RowDefinitions="Auto,Auto,Auto,Auto,Auto,Auto">
				<TextBlock Margin="0 1" Theme="{StaticResource CaptionTextBlockStyle}" Text="Name" Padding="0" Grid.Row="0" IsVisible="{Binding HasArtifactName}"/>
				<TextBlock Margin="42 1" Theme="{StaticResource CaptionTextBlockStyle}" Text="{Binding ArtifactName}" Padding="0" Grid.Row="0" Grid.Column="1" IsVisible="{Binding HasArtifactName}"/>

				<TextBlock Margin="0 1" Theme="{StaticResource CaptionTextBlockStyle}" Text="Type" Padding="0" Grid.Row="1" IsVisible="{Binding HasArtifactType}"/>
				<TextBlock Margin="42 1" Theme="{StaticResource CaptionTextBlockStyle}" Text="{Binding ArtifactType}" Padding="0" Grid.Row="1" Grid.Column="1" IsVisible="{Binding HasArtifactType}"/>

				<TextBlock Margin="0 1" Theme="{StaticResource CaptionTextBlockStyle}" Text="Description" Padding="0" Grid.Row="2" IsVisible="{Binding HasArtifactDescription}"/>
				<TextBlock Margin="42 1" Theme="{StaticResource CaptionTextBlockStyle}" Text="{Binding ArtifactDescription}" Padding="0" Grid.Row="2" Grid.Column="1" IsVisible="{Binding HasArtifactDescription}"/>

				<TextBlock Margin="0 1" Theme="{StaticResource CaptionTextBlockStyle}" Text="Job" Padding="0" Grid.Row="3" IsVisible="{Binding HasArtifactJobUrl}"/>
				<HyperlinkButton Margin="41 1" FontSize="11" Content="{Binding ArtifactJobUrl}" Padding="0" Grid.Row="3" Grid.Column="1" Command="{Binding OpenJobUrl}" IsVisible="{Binding HasArtifactJobUrl}"/>

				<TextBlock Margin="0 1" Theme="{StaticResource CaptionTextBlockStyle}" Text="Keys" Padding="0" Grid.Row="4"/>
				<TextBlock Margin="42 1" Theme="{StaticResource CaptionTextBlockStyle}" Text="{Binding ArtifactKeys}" Padding="0" Grid.Row="4" Grid.Column="1"/>

				<TextBlock Margin="0 2" Theme="{StaticResource CaptionTextBlockStyle}" Text="Metadata" Padding="0" Grid.Row="5"/>
				<TextBlock Margin="42 1" Theme="{StaticResource CaptionTextBlockStyle}" Text="{Binding ArtifactMetadata}" Padding="0" Grid.Column="1" Grid.Row="5"/>
			</Grid>
		</Border>

		<TextBlock Classes="SettingHeader">Output</TextBlock>

		<Border Classes="Card">
			<StackPanel Spacing="4" Margin="0">
				<Grid ColumnDefinitions="*,Auto" Grid.Column="2" Grid.Row="1">
					<HyperlinkButton Content="{Binding OutputDir}" Command="{Binding OpenOutputDir}" Margin="0" Padding="0"/>
					<Button Margin="16 0 0 0" Grid.Column="1" Command="{Binding SelectOutputDirAsync}" Width="120">Browse...</Button>
				</Grid>
				<CheckBox IsChecked="{Binding AppendBuildName}" Content="Append build name"/>
				<CheckBox IsChecked="{Binding PatchExistingData}" Content="Reuse existing data for patching"/>
			</StackPanel>
		</Border>

		<StackPanel Margin="0" Spacing="5" IsVisible="{Binding ShowErrorMessage}">
			<TextBlock Classes="SettingHeader">Error</TextBlock>

			<Border Classes="Card">
				<TextBlock Text="{Binding ErrorMessage}" Foreground="Red" TextWrapping="Wrap"/>
			</Border>
		</StackPanel>

		<Grid ColumnDefinitions="*,*" HorizontalAlignment="Right">
			<Button Grid.Column="0" Margin="5 10 10 0" Width="100" Click="Start_Click" IsEnabled="{Binding AllowStart}">Start</Button>
			<Button Grid.Column="1" Margin="5 10 0 0" Width="100" Click="Cancel_Click">Cancel</Button>
		</Grid>

	</StackPanel>
</Window>
