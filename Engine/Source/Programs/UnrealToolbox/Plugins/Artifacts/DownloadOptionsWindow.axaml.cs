// Copyright Epic Games, Inc. All Rights Reserved.

using Avalonia.Controls;
using Avalonia.Interactivity;

namespace UnrealToolbox.Plugins.Artifacts
{
	partial class DownloadOptionsWindow : Window
	{
		readonly DownloadOptionsViewModel _viewModel;
		readonly JsonConfig<DownloadSettings>? _settings;
		readonly TaskCompletionSource<DownloadOptions?> _result = new TaskCompletionSource<DownloadOptions?>();

		public Task<DownloadOptions?> Result => _result.Task;

		public DownloadOptionsWindow()
			: this(null, null, null)
		{ }

		public DownloadOptionsWindow(Uri? artifactUrl, JsonConfig<DownloadSettings>? settings, IHordeClientProvider? hordeClientProvider)
		{
			_settings = settings;

			_viewModel = new DownloadOptionsViewModel(this, _settings?.Current, hordeClientProvider);
			_viewModel.ArtifactUrl = artifactUrl;

			DataContext = _viewModel;

			InitializeComponent();
		}

		protected override void OnOpened(EventArgs e)
		{
			base.OnOpened(e);

			if (_viewModel != null && _viewModel.ArtifactUrl != null)
			{
				_ = _viewModel.RefreshArtifactAsync();
			}
		}

		public async Task SelectInputFileAsync()
		{
			await _viewModel.SelectInputFileAsync();
		}

		void Start_Click(object? sender, RoutedEventArgs args)
		{
			DownloadOptions? options = _viewModel.GetDownloadOptions();
			if (options != null)
			{
				if (_settings != null)
				{
					DownloadSettings newSettings = new DownloadSettings(_settings.Current);
					newSettings.OutputDir = _viewModel.BaseOutputDir;
					newSettings.AppendBuildName = _viewModel.AppendBuildName;
					newSettings.PatchExistingData = _viewModel.PatchExistingData;
					_settings.UpdateSettings(newSettings);
				}

				_result.TrySetResult(options);
				Close();
			}
		}

		void Cancel_Click(object? sender, RoutedEventArgs args)
		{
			_result.TrySetResult(null);
			Close();
		}
	}
}