// Copyright Epic Games, Inc. All Rights Reserved.

using Avalonia.Threading;
using CommunityToolkit.Mvvm.ComponentModel;
using EpicGames.Horde.Storage.Nodes;

namespace UnrealToolbox.Plugins.Artifacts
{
	partial class DownloadProgressViewModel : ObservableObject, IProgress<IExtractStats>
	{
		long _size;
		long _totalSize;

		[ObservableProperty]
		string _status = "Starting...";

		[ObservableProperty]
		bool _isRunning = true;

		[ObservableProperty]
		int _progress;

		[ObservableProperty]
		bool _isProgressBarVisible;

		[ObservableProperty]
		[NotifyPropertyChangedFor(nameof(ShowErrorMessage))]
		[NotifyPropertyChangedFor(nameof(CloseText))]
		string _errorMessage = String.Empty;

		[ObservableProperty]
		[NotifyPropertyChangedFor(nameof(ShowOpenFolder))]
		string? _openFolderPath;

		public bool ShowErrorMessage
			=> ErrorMessage.Length > 0;

		public bool ShowOpenFolder
			=> !String.IsNullOrEmpty(OpenFolderPath);

		public string CloseText
			=> ShowErrorMessage ? "Close" : "Cancel";

		public void Report(IExtractStats value)
		{
			string status = $"Copied {value.NumFiles} files ({value.DownloadSize / (1024.0 * 1024.0):n1}mb, {value.DownloadRate / (1024.0 * 1024.0):n1}mb/s)...";
			long size = value.DownloadSize;
			Dispatcher.UIThread.Invoke(() => ReportOnMainThread(status, size));
		}

		public void MarkComplete()
		{
			_size = _totalSize;
			UpdateProgress();
		}

		void ReportOnMainThread(string status, long size)
		{
			Status = status;
			_size = size;
			UpdateProgress();
		}

		public void SetTotalSize(long totalSize)
		{
			_totalSize = totalSize;
			UpdateProgress();
		}

		void UpdateProgress()
		{
			if (_totalSize != 0)
			{
				Progress = (int)((_size * 100) / Math.Max(_totalSize, 1));
				IsProgressBarVisible = true;
			}
		}
	}
}
