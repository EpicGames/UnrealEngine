<Window xmlns="https://github.com/avaloniaui"
        xmlns:x="http://schemas.microsoft.com/winfx/2006/xaml"
        xmlns:d="http://schemas.microsoft.com/expression/blend/2008"
        xmlns:mc="http://schemas.openxmlformats.org/markup-compatibility/2006"
		xmlns:ui="using:FluentAvalonia.UI.Controls"
        xmlns:local="using:UnrealToolbox.Plugins.Artifacts"
        mc:Ignorable="d"
        x:Class="UnrealToolbox.Plugins.Artifacts.DownloadProgressWindow"
		x:DataType="local:DownloadProgressViewModel"
        Title="Downloading Artifact"
		Width="700"
		CanResize="False"
		SizeToContent="Height"
		WindowStartupLocation="CenterScreen">

	<Window.Styles>
		<StyleInclude Source="/Styles.axaml"/>
	</Window.Styles>

	<StackPanel Spacing="20" Margin="32 20">
		<Grid ColumnDefinitions="*,Auto,Auto" RowDefinitions="Auto,Auto,Auto">
			<Grid Grid.Column="0" Grid.Row="0" ColumnDefinitions="Auto,*">
				<ui:ProgressRing Grid.Column="0" Grid.Row="0" Width="16" Height="16" Margin="0 2 6 2" IsVisible="{Binding IsRunning}" VerticalAlignment="Center"/>
				<TextBlock Grid.Column="1" Grid.Row="0" VerticalAlignment="Center" Theme="{DynamicResource BodyTextBlockStyle}" HorizontalAlignment="Left" Text="{Binding Status}"/>
			</Grid>
			<ProgressBar Grid.Column="0" Grid.Row="1" Value="{Binding Progress}" Margin="0 0 12 0"/>
			<Button Grid.Column="1" Grid.RowSpan="2" Margin="8 0 0 0" VerticalAlignment="Center" Content="Open Folder" IsVisible="{Binding ShowOpenFolder}" Click="OpenFolder"/>
			<Button Grid.Column="2" Grid.RowSpan="2" Margin="8 0 0 0" VerticalAlignment="Center" Content="{Binding CloseText}" Width="100" Click="Cancel"/>
		</Grid>

		<Border Classes="Card" IsVisible="{Binding ShowErrorMessage}">
			<ScrollViewer>
				<SelectableTextBlock Text="{Binding ErrorMessage}" Foreground="Red" />
			</ScrollViewer>
		</Border>
	</StackPanel>
</Window>
