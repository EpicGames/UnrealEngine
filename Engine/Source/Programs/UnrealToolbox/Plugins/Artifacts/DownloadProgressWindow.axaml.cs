// Copyright Epic Games, Inc. All Rights Reserved.

using System.Diagnostics;
using Avalonia.Controls;
using Avalonia.Interactivity;

namespace UnrealToolbox.Plugins.Artifacts
{
	partial class DownloadProgressWindow : Window
	{
		readonly DownloadProgressViewModel _viewModel;

		public DownloadProgressWindow()
			: this(new DownloadProgressViewModel())
		{
		}

		public DownloadProgressWindow(DownloadProgressViewModel viewModel)
		{
			InitializeComponent();

			_viewModel = viewModel;
			DataContext = viewModel;
		}

		public void OpenFolder(object? sender, RoutedEventArgs args)
		{
			if (!String.IsNullOrEmpty(_viewModel.OpenFolderPath))
			{
				Process.Start(new ProcessStartInfo { FileName = _viewModel.OpenFolderPath, UseShellExecute = true });
				Close();
			}
		}

		public void Cancel(object? sender, RoutedEventArgs args)
		{
			Close();
		}
	}
}
