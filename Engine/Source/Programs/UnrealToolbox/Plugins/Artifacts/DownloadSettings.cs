// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealToolbox.Plugins.Artifacts
{
	/// <summary>
	/// Settings for the artifacts plugin
	/// </summary>
	[Serializable]
	public class DownloadSettings
	{
		/// <summary>
		/// Output directory for the download
		/// </summary>
		public string? OutputDir { get; set; }

		/// <summary>
		/// Whether to append the build name to the output directory
		/// </summary>
		public bool AppendBuildName { get; set; }

		/// <summary>
		/// Whether to patch existing data when writing the new build
		/// </summary>
		public bool PatchExistingData { get; set; }

		/// <summary>
		/// Default constructor
		/// </summary>
		public DownloadSettings()
		{ }

		/// <summary>
		/// Copy constructor
		/// </summary>
		public DownloadSettings(DownloadSettings other)
		{
			OutputDir = other.OutputDir;
			AppendBuildName = other.AppendBuildName;
			PatchExistingData = other.PatchExistingData;
		}
	}
}
