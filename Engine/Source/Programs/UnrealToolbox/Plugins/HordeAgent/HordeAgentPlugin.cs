// Copyright Epic Games, Inc. All Rights Reserved.

using Avalonia.Controls;
using EpicGames.Core;
using EpicGames.Horde;
using FluentAvalonia.UI.Controls;
using HordeAgent;
using Microsoft.Extensions.Logging;
using Microsoft.Win32;
using System.Diagnostics;
using System.IO.Pipes;
using System.Runtime.InteropServices;
using System.Text.Json;
using System.Text.Json.Serialization;

namespace UnrealToolbox.Plugins.HordeAgent
{
	class HordeAgentPlugin : ToolboxPluginBase
	{
		record struct IdleStat(string Name, long Value, long MinValue);

		readonly IToolboxPluginHost _host;
		readonly IToolCatalog _toolCatalog;
		readonly ILogger _logger;

		readonly BackgroundTask _clientTask;
		readonly BackgroundTask _tickPauseStateTask;

		readonly FileReference _settingsFile;

		byte[] _settingsData = Array.Empty<byte>();
		HordeAgentSettings _settings = new HordeAgentSettings();
		bool _pipeConnected;

		public HordeAgentSettings Settings => _settings;

		AgentSettingsMessage? _agentSettings;

		ToolboxPluginStatus? _status;
		ToolboxPluginStatus? _reportStatus; // Updated with _status when nothing is currently being installed

		public override string Name => "Horde Agent";

		public override IconSource Icon => new SymbolIconSource() { Symbol = Symbol.People };

		public bool IsEnabled { get; private set; }

		public override bool HasSettingsPage()
			=> IsEnabled;

		public override Control CreateSettingsPage(SettingsContext context)
			=> new HordeAgentSettingsPage(context, this);

		public void UpdateSettings(HordeAgentSettings settings)
		{
			byte[] data = JsonSerializer.SerializeToUtf8Bytes(settings, GetJsonSerializerOptions());
			if (!data.SequenceEqual(_settingsData))
			{
				_settings = settings;
				_settingsData = data;

				DirectoryReference.CreateDirectory(_settingsFile.Directory);
				FileReference.WriteAllBytes(_settingsFile, data);

				_statusChangedEvent.Set();
			}
		}

		void EnrollWithServer()
		{
			Uri? serverUrl = _agentSettings?.ServerUrl;
			if (serverUrl != null)
			{
				Process.Start(new ProcessStartInfo(new Uri(serverUrl, "agents/registration").ToString()) { UseShellExecute = true });
			}
		}

		public HordeAgentPlugin(IToolboxPluginHost host, ToolCatalog toolCatalog, ILogger<HordeAgentPlugin> logger)
		{
			_host = host;
			_toolCatalog = toolCatalog;
			_logger = logger;

			_settingsFile = FileReference.Combine(Program.DataDir, "HordeAgent.json");

			LoadSettings();

			_clientTask = BackgroundTask.StartNew(StatusTaskAsync);
			_tickPauseStateTask = BackgroundTask.StartNew(ctx => TickPauseStateAsync(ctx));
		}

		public override bool Refresh()
		{
			if (OperatingSystem.IsWindows())
			{
				bool enabled = _pipeConnected || ((Registry.GetValue("HKEY_LOCAL_MACHINE\\SOFTWARE\\Epic Games\\Horde\\Agent", "Installed", null) as int?) ?? 0) != 0;
				if (enabled != IsEnabled)
				{
					IsEnabled = enabled;
					return true;
				}
			}
			return false;
		}

		static JsonSerializerOptions GetJsonSerializerOptions()
		{
			JsonSerializerOptions options = new JsonSerializerOptions();
			options.PropertyNamingPolicy = JsonNamingPolicy.CamelCase;
			options.PropertyNameCaseInsensitive = true;
			options.DefaultIgnoreCondition = JsonIgnoreCondition.WhenWritingNull;
			options.AllowTrailingCommas = true;
			options.WriteIndented = true;
			options.Converters.Add(new JsonStringEnumConverter());
			return options;
		}

		void LoadSettings()
		{
			if (FileReference.Exists(_settingsFile))
			{
				try
				{
					byte[] data = FileReference.ReadAllBytes(_settingsFile);
					if (!data.SequenceEqual(_settingsData))
					{
						_settings = JsonSerializer.Deserialize<HordeAgentSettings>(data, GetJsonSerializerOptions())!;
						if (_settings.Mode == null)
						{
							_settings.Mode = ReadLegacyMode();
							UpdateSettings(_settings);
						}
						_settingsData = data;
						return;
					}
				}
				catch (Exception ex)
				{
					_logger.LogError(ex, "Error while reading {File}: {Message}", _settingsFile, ex.Message);
				}
			}
			else
			{
				DirectoryReference? programData = DirectoryReference.GetSpecialFolder(Environment.SpecialFolder.CommonApplicationData);
				if (programData != null)
				{
					FileReference legacySettingsFile = FileReference.Combine(programData, "Epic", "Horde", "TrayApp", "settings.json");
					if (FileReference.Exists(legacySettingsFile))
					{
						try
						{
							byte[] data = FileReference.ReadAllBytes(legacySettingsFile);

							HordeAgentSettings settings = JsonSerializer.Deserialize<HordeAgentSettings>(data, GetJsonSerializerOptions())!;
							settings.Mode ??= ReadLegacyMode();

							UpdateSettings(settings);
							return;
						}
						catch (Exception ex)
						{
							_logger.LogError(ex, "Error while reading {File}: {Message}", legacySettingsFile, ex.Message);
						}
					}
				}
			}

			UpdateSettings(new HordeAgentSettings());
		}

		static AgentMode? ReadLegacyMode()
		{
			if (!OperatingSystem.IsWindows())
			{
				return null;
			}

			const string RegistryKey = "HKEY_CURRENT_USER\\Software\\Epic Games\\Horde\\TrayApp";
			const string RegistryStatusValue = "Status";

			int? status = Registry.GetValue(RegistryKey, RegistryStatusValue, null) as int?;
			if (status == null)
			{
				return null;
			}

			return status.Value switch
			{
				0 => AgentMode.Dedicated,
				1 => AgentMode.Disabled,
				2 => AgentMode.Workstation,
				_ => null
			};
		}

		public override void PopulateContextMenu(NativeMenu contextMenu)
		{
			if (IsEnabled)
			{
				NativeMenuItem enrollMenuItem = new NativeMenuItem("Enroll with Server...");
				enrollMenuItem.Click += (s, e) => EnrollWithServer();
				contextMenu.Items.Add(enrollMenuItem);
			}
		}

		public override async ValueTask DisposeAsync()
		{
			await base.DisposeAsync();

			await _tickPauseStateTask.DisposeAsync();
			await _clientTask.DisposeAsync();
		}

		private void OnOpenLogs(object? sender, EventArgs e)
		{
			DirectoryReference? programDataDir = DirectoryReference.GetSpecialFolder(Environment.SpecialFolder.CommonApplicationData);
			if (programDataDir != null)
			{
				DirectoryReference logsDir = DirectoryReference.Combine(programDataDir, "Epic", "Horde", "Agent");
				if (DirectoryReference.Exists(logsDir))
				{
					Process.Start(new ProcessStartInfo { FileName = logsDir.FullName, UseShellExecute = true });
				}
			}
		}

		void SetStatus(AgentStatusMessage status)
		{
			if (!IsEnabled)
			{
				_status = null;
			}
			else if (!status.Healthy)
			{				
				string message = String.IsNullOrEmpty(status.Detail) ? "Error. Check logs." : status.Detail.Length > 100 ? status.Detail.Substring(0, 100) : status.Detail;
				_status = new ToolboxPluginStatus(TrayAppPluginState.Error, message);
				
				if (_settings.Mode != AgentMode.Disabled)
				{
					// Make some known error messages more friendly
					if (message.Contains("actively refused it", StringComparison.OrdinalIgnoreCase))
					{
						message = $"Could not connect to Horde Server: {HordeOptions.GetDefaultServerUrl()?.ToString() ?? "(Not configured)"}";
					}

					if (message.Contains("enrollment key does not match", StringComparison.OrdinalIgnoreCase))
					{
						message = $"Agent registration revoked by Horde Server: {HordeOptions.GetDefaultServerUrl()?.ToString() ?? "(Not configured)"}";
					}

					ToolboxNotificationManager.PostNotification("Horde Agent", message);
				}
			}
			else if (status.NumLeases > 0)
			{
				string message = status.NumLeases == 1 ? "Currently handling 1 lease" : $"Currently handling {status.NumLeases} leases";
				_status = new ToolboxPluginStatus(TrayAppPluginState.Busy, message);
			}
			else if (_enabled)
			{
				string message = "Agent is operating normally";
				_status = new ToolboxPluginStatus(TrayAppPluginState.Ok, message);
			}
			else
			{
				string message = "Agent is paused";
				_status = new ToolboxPluginStatus(TrayAppPluginState.Paused, message);
			}
			_host.UpdateStatus();
		}

		public override ToolboxPluginStatus GetStatus()
		{
			if (_settings.Mode == AgentMode.Disabled)
			{
				return ToolboxPluginStatus.Default;
			}

			if (!_toolCatalog.Items.Any(x => x.Pending != null && !x.Pending.Failed))
			{
				_reportStatus = _status;
			}
			return _reportStatus ?? ToolboxPluginStatus.Default;
		}

		async Task StatusTaskAsync(CancellationToken cancellationToken)
		{
			SetStatus(new AgentStatusMessage(true, 0, AgentStatusMessage.Starting));
			for (; ; )
			{
				try
				{
					try
					{
						await PollForStatusUpdatesAsync(cancellationToken);
					}
					finally
					{
						_pipeConnected = false;
					}
				}
				catch (OperationCanceledException) when (cancellationToken.IsCancellationRequested)
				{
					break;
				}
				catch
				{
					SetStatus(new AgentStatusMessage(false, 0, "Unable to connect to Agent. Check the service is running."));
					await Task.Delay(TimeSpan.FromSeconds(5.0), cancellationToken);
				}
			}
		}

#pragma warning disable IDE1006
		[StructLayout(LayoutKind.Sequential)]
		struct LASTINPUTINFO
		{
			public int cbSize;
			public uint dwTime;
		}

		[DllImport("user32.dll")]
		static extern bool GetLastInputInfo(ref LASTINPUTINFO plii);

		[DllImport("kernel32.dll")]
		static extern uint GetTickCount();

		[StructLayout(LayoutKind.Sequential)]
		struct FILETIME
		{
			public uint dwLowDateTime;
			public uint dwHighDateTime;

			public readonly ulong Total => dwLowDateTime | (ulong)dwHighDateTime << 32;
		};

		[StructLayout(LayoutKind.Sequential)]
		struct MEMORYSTATUSEX
		{
			public int dwLength;
			public uint dwMemoryLoad;
			public ulong ullTotalPhys;
			public ulong ullAvailPhys;
			public ulong ullTotalPageFile;
			public ulong ullAvailPageFile;
			public ulong ullTotalVirtual;
			public ulong ullAvailVirtual;
			public ulong ullAvailExtendedVirtual;
		}

		[DllImport("kernel32.dll", CharSet = CharSet.Auto, SetLastError = true)]
		static extern bool GlobalMemoryStatusEx(ref MEMORYSTATUSEX lpBuffer);

		[DllImport("kernel32.dll", SetLastError = true)]
		static extern bool GetSystemTimes(out FILETIME lpIdleTime, out FILETIME lpKernelTime, out FILETIME lpUserTime);
#pragma warning restore IDE1006

		bool _enabled;
		readonly AsyncEvent _statusChangedEvent = new AsyncEvent();
		readonly AsyncEvent _enabledChangedEvent = new AsyncEvent();

		async Task TickPauseStateAsync(CancellationToken cancellationToken)
		{
			await using BackgroundTask cpuStatsTask = BackgroundTask.StartNew(ctx => TickCpuStatsAsync(ctx));
			await using BackgroundTask criticalProcessTask = BackgroundTask.StartNew(ctx => TickCriticalProcessAsync(ctx));

			TimeSpan pollInterval = TimeSpan.FromSeconds(0.25);

			Stopwatch stateChangeTimer = Stopwatch.StartNew();
			while (!cancellationToken.IsCancellationRequested)
			{
				Task statusChangedTask = _statusChangedEvent.Task;

				AgentMode mode = _settings.Mode ?? AgentMode.Disabled;
				if (mode == AgentMode.Dedicated)
				{
					if (!_enabled)
					{
						_enabled = true;
						_enabledChangedEvent.Set();
					}
				}
				else if (mode == AgentMode.Disabled)
				{
					if (_enabled)
					{
						_enabled = false;
						_enabledChangedEvent.Set();
					}
				}

				DateTime utcNow = DateTime.UtcNow;
				IEnumerable<IdleStat> idleStats = GetIdleStats();

				bool idle = idleStats.All(x => x.Value >= x.MinValue);
				if (idle == _enabled)
				{
					stateChangeTimer.Restart();
				}

				const int WakeTimeSecs = 2;
				const int IdleTimeSecs = 30;
				int stateChangeTime = (int)stateChangeTimer.Elapsed.TotalSeconds;
				int stateChangeMaxTime = _enabled ? WakeTimeSecs : IdleTimeSecs;
				//				_idleForm?.TickStats(_enabled, stateChangeTime, stateChangeMaxTime, idleStats);

				if (mode == AgentMode.Workstation && stateChangeTime >= stateChangeMaxTime)
				{
					_enabled ^= true;
					_enabledChangedEvent.Set();
					stateChangeTimer.Restart();
				}

				await Task.WhenAny(statusChangedTask, Task.Delay(pollInterval, cancellationToken));
			}
		}

		IEnumerable<IdleStat> GetIdleStats()
		{
			// Check there has been no input for a while
			LASTINPUTINFO lastInputInfo = new LASTINPUTINFO();
			lastInputInfo.cbSize = Marshal.SizeOf<LASTINPUTINFO>();

			if (GetLastInputInfo(ref lastInputInfo))
			{
				yield return new IdleStat("LastInputTime", (GetTickCount() - lastInputInfo.dwTime) / 1000, _settings.Idle.MinIdleTimeSecs);
			}

			// Check that no critical processes are running
			if (_settings.Idle.CriticalProcesses.Any())
			{
				yield return new IdleStat("CriticalProcCount", -_idleCriticalProcessCount, 0);
			}

			// Only look at memory/CPU usage if we're not paused; executing jobs will increase them
			if (!_enabled)
			{
				// Check the CPU usage doesn't exceed the limit
				yield return new IdleStat("IdleCpuPct", _idleCpuPct, _settings.Idle.MinIdleCpuPct);

				// Check there's enough available virtual memory 
				MEMORYSTATUSEX memoryStatus = new MEMORYSTATUSEX();
				memoryStatus.dwLength = Marshal.SizeOf<MEMORYSTATUSEX>();

				if (GlobalMemoryStatusEx(ref memoryStatus))
				{
					yield return new IdleStat("VirtualMemMb", (long)(memoryStatus.ullAvailPhys + memoryStatus.ullAvailPageFile) / (1024 * 1024), _settings.Idle.MinFreeVirtualMemMb);
				}
			}
		}

		int _idleCpuPct = 0;
		int _idleCriticalProcessCount = 0;

		async Task TickCpuStatsAsync(CancellationToken cancellationToken)
		{
			const int NumSamples = 10;
			TimeSpan sampleInterval = TimeSpan.FromSeconds(0.2);
			(ulong IdleTime, ulong TotalTime)[] samples = new (ulong IdleTime, ulong TotalTime)[NumSamples];

			int sampleIdx = 0;
			for (; ; )
			{
				if (GetSystemTimes(out FILETIME idleTime, out FILETIME kernelTime, out FILETIME userTime))
				{
					(ulong prevIdleTime, ulong prevTotalTime) = samples[sampleIdx];
					(ulong nextIdleTime, ulong nextTotalTime) = (idleTime.Total, kernelTime.Total + userTime.Total);

					samples[sampleIdx] = (nextIdleTime, nextTotalTime);
					sampleIdx = (sampleIdx + 1) % NumSamples;

					if (prevTotalTime > 0 && nextTotalTime > prevTotalTime)
					{
						_idleCpuPct = (int)((nextIdleTime - prevIdleTime) * 100 / (nextTotalTime - prevTotalTime));
					}
				}
				await Task.Delay(sampleInterval, cancellationToken);
			}
		}

		async Task TickCriticalProcessAsync(CancellationToken cancellationToken)
		{
			TimeSpan sampleInterval = TimeSpan.FromSeconds(1.0);

			for (; ; )
			{
				try
				{
					if (RuntimeInformation.IsOSPlatform(OSPlatform.Windows) && _settings.Idle.CriticalProcesses.Any())
					{
						IEnumerable<int> hordeProcessIds = Process.GetProcessesByName("HordeAgent").Select(x => x.Id);
						IEnumerable<Process> criticalProcesses = _settings.Idle.CriticalProcesses
							.Select(x => Path.GetFileNameWithoutExtension(x).ToUpperInvariant())
							.Distinct()
							.SelectMany(x => Process.GetProcessesByName(x))
							.Where(x => !x.HasExited);

						// Ignore processes that are descendants of HordeAgent
						if (hordeProcessIds.Any() && criticalProcesses.Any())
						{
							criticalProcesses = criticalProcesses
								.Where(x => !ProcessUtils.GetAncestorProcesses(x)
								.Select(x => x.Id).Intersect(hordeProcessIds).Any())
								.Where(x => !x.HasExited);
						}

						_idleCriticalProcessCount = criticalProcesses.Count();
					}
				}
				catch (InvalidOperationException)
				{
					// If a process stops running Process.Id will throw an exception
				}
				await Task.Delay(sampleInterval, cancellationToken);
			}
		}

		async Task PollForStatusUpdatesAsync(CancellationToken cancellationToken)
		{
			AgentMessageBuffer message = new AgentMessageBuffer();
			using (NamedPipeClientStream pipeClient = new NamedPipeClientStream(".", AgentMessagePipe.PipeName, PipeDirection.InOut))
			{
				SetStatus(new AgentStatusMessage(true, 0, "Connecting to agent..."));
				await pipeClient.ConnectAsync(cancellationToken);

				_pipeConnected = true;

				SetStatus(new AgentStatusMessage(true, 0, "Waiting for status update."));
				for (; ; )
				{
					Task idleChangeTask = _enabledChangedEvent.Task;

					bool enabled = _enabled;
					message.Set(AgentMessageType.SetEnabledRequest, new AgentEnabledMessage(enabled));
					await message.SendAsync(pipeClient, cancellationToken);

					if (_agentSettings == null)
					{
						message.Set(AgentMessageType.GetSettingsRequest);
						await message.SendAsync(pipeClient, cancellationToken);

						if (!await message.TryReadAsync(pipeClient, cancellationToken))
						{
							break;
						}
						if (message.Type == AgentMessageType.GetSettingsResponse)
						{
							_agentSettings = message.Parse<AgentSettingsMessage>();
						}
					}
					
					message.Set(AgentMessageType.SetSettingsRequest, new AgentSetSettingsRequest(_settings.Cpu.CpuCount, _settings.Cpu.CpuMultiplier));
					await message.SendAsync(pipeClient, cancellationToken);

					message.Set(AgentMessageType.GetStatusRequest);
					await message.SendAsync(pipeClient, cancellationToken);

					if (!await message.TryReadAsync(pipeClient, cancellationToken))
					{
						break;
					}

					switch (message.Type)
					{
						case AgentMessageType.GetStatusResponse:
							AgentStatusMessage status = message.Parse<AgentStatusMessage>();
							SetStatus(status);
							break;
					}

					await Task.WhenAny(idleChangeTask, Task.Delay(TimeSpan.FromSeconds(5.0), cancellationToken));
				}
			}
		}
	}
}
