// Copyright Epic Games, Inc. All Rights Reserved.

#pragma warning disable CS1591

namespace UnrealToolbox.Plugins.HordeAgent
{
	public enum AgentMode
	{
		Workstation = 0,
		Dedicated = 1,
		Disabled = 2,
	}

	[Serializable]
	public class HordeAgentSettings
	{
		public AgentMode? Mode { get; set; }
		public IdleSettings Idle { get; set; } = new();
		public CpuUtilizationSettings Cpu { get; set; } = new();
	}

	[Serializable]
	public class IdleSettings
	{
		public int MinIdleTimeSecs { get; set; } = 2;
		public int MinIdleCpuPct { get; set; } = 70;
		public int MinFreeVirtualMemMb { get; set; } = 256;

		public string[] CriticalProcesses { get; set; } = Array.Empty<string>();
	}
	
	[Serializable]
	public class CpuUtilizationSettings
	{
		public int CpuCount { get; set; } = Environment.ProcessorCount;
		public double CpuMultiplier { get; set; } = 1.0;
	}
}
