<UserControl xmlns="https://github.com/avaloniaui"
		xmlns:x="http://schemas.microsoft.com/winfx/2006/xaml"
		xmlns:d="http://schemas.microsoft.com/expression/blend/2008"
		xmlns:mc="http://schemas.openxmlformats.org/markup-compatibility/2006"
		xmlns:local="using:UnrealToolbox.Plugins.HordeAgent"
		mc:Ignorable="d" d:DesignWidth="800" d:DesignHeight="1000"
		x:Class="UnrealToolbox.Plugins.HordeAgent.HordeAgentSettingsPage"
		x:DataType="local:HordeAgentSettingsViewModel"
		Background="{DynamicResource SolidBackgroundFillColorBaseBrush}">

	<UserControl.Styles>
		<StyleInclude Source="/Styles.axaml"/>
	</UserControl.Styles>

	<ScrollViewer>
		<StackPanel Margin="20" Spacing="5">

			<TextBlock Theme="{StaticResource SubtitleTextBlockStyle}" Margin="0 0 0 0">Horde Agent</TextBlock>

			<TextBlock Classes="SettingHeader">Mode</TextBlock>

			<Border Classes="Card">
				<StackPanel Spacing="10" Margin="0 5 0 10">
					<RadioButton Margin="10 0" IsChecked="{Binding IsDedicatedMode}">
						<StackPanel>
							<TextBlock Classes="SettingCaption">Dedicated</TextBlock>
							<TextBlock Classes="SettingHelp">This machine is a dedicated build agent, and should continue to execute work even when in use.</TextBlock>
						</StackPanel>
					</RadioButton>
					<RadioButton Margin="10 0" IsChecked="{Binding IsWorkstationMode}">
						<StackPanel>
							<TextBlock Classes="SettingCaption">Workstation</TextBlock>
							<TextBlock Classes="SettingHelp">Only offer this machine's idle CPU cycles for other compute work when it is not actively being used.</TextBlock>
						</StackPanel>
					</RadioButton>
					<RadioButton Margin="10 0" IsChecked="{Binding IsDisabled}">
						<StackPanel>
							<TextBlock Classes="SettingCaption">Disabled</TextBlock>
							<TextBlock Classes="SettingHelp">This agent will not execute any workloads.</TextBlock>
						</StackPanel>
					</RadioButton>
				</StackPanel>
			</Border>

			<!--
			<TextBlock Classes="SettingHeader">Sandbox</TextBlock>

			<Border Classes="Card">
				<Grid ColumnDefinitions="*,Auto">
					<HyperlinkButton Grid.Column="0" Command="{Binding OpenSandbox}" VerticalAlignment="Center" Content="{Binding SandboxDir}"/>
					<Button IsVisible="False" Grid.Column="1" VerticalAlignment="Center" Command="{Binding MoveSandboxAsync}">Modify</Button>
				</Grid>
			</Border>
			-->

			<TextBlock Classes="SettingHeader" IsVisible="{Binding IsWorkstationMode}">Idle Detection</TextBlock>

			<Border Classes="Card" IsVisible="{Binding IsWorkstationMode}">
				<Grid ColumnDefinitions="*,Auto">
					<StackPanel Grid.Column="0">
						<TextBlock Classes="SettingCaption">Minimum idle time</TextBlock>
						<TextBlock Classes="SettingHelp">Time that the workstation have been idle before it will transition to helping</TextBlock>
					</StackPanel>
					<NumericUpDown VerticalAlignment="Center" Grid.Column="1" Minimum="0" Maximum="86400" FormatString="0 secs" Width="160" Value="{Binding MinIdleTimeSecs}"/>
				</Grid>
			</Border>

			<Border Classes="Card" IsVisible="{Binding IsWorkstationMode}">
				<Grid ColumnDefinitions="*,Auto">
					<StackPanel Grid.Column="0">
						<TextBlock Classes="SettingCaption">Minimum free virtual memory</TextBlock>
						<TextBlock Classes="SettingHelp">Amount of free virtual memory required for the workstation to take on work</TextBlock>
					</StackPanel>
					<NumericUpDown Grid.Column="1" Minimum="0" Maximum="20000" FormatString="0 MB" Width="160" VerticalAlignment="Center" Value="{Binding MinFreeMemoryMb}"/>
				</Grid>
			</Border>

			<Border Classes="Card" IsVisible="{Binding IsWorkstationMode}">
				<Grid ColumnDefinitions="*,Auto">
					<StackPanel Grid.Column="0">
						<TextBlock Classes="SettingCaption">Minimum free CPU</TextBlock>
						<TextBlock Classes="SettingHelp">Percentage of unused CPU capacity for the machine to be considered idle</TextBlock>
					</StackPanel>
					<NumericUpDown Grid.Column="1" Minimum="0" Maximum="100" FormatString="0 \%" Width="160" VerticalAlignment="Center" Value="{Binding MinFreeCpuPct}"/>
				</Grid>
			</Border>

			<TextBlock Classes="SettingHeader" IsVisible="{Binding IsWorkstationMode}">CPU Utilization</TextBlock>

			<Border Classes="Card" IsVisible="{Binding IsWorkstationMode}">
				<Grid ColumnDefinitions="*,Auto">
					<StackPanel Grid.Column="0">
						<TextBlock Classes="SettingCaption">CPU Count</TextBlock>
						<TextBlock Classes="SettingHelp">Maximum number of logical CPU cores workloads should use</TextBlock>
					</StackPanel>
					<NumericUpDown x:Name="CpuCount" VerticalAlignment="Center" Grid.Column="1" Minimum="0" Maximum="512" FormatString="0 cores" Width="160" Value="{Binding CpuCount}"/>
				</Grid>
			</Border>

			<Border Classes="Card" IsVisible="{Binding IsWorkstationMode}">
				<Grid ColumnDefinitions="*,Auto">
					<StackPanel Grid.Column="0">
						<TextBlock Classes="SettingCaption">CPU Multiplier</TextBlock>
						<TextBlock Classes="SettingHelp">Multiplier applied to CPU core count</TextBlock>
					</StackPanel>
					<NumericUpDown Grid.Column="1" Minimum="0" Maximum="20" Increment="0.1" FormatString="0.0x" Width="160" VerticalAlignment="Center" Value="{Binding CpuMultiplier}"/>
				</Grid>
			</Border>

		</StackPanel>
	</ScrollViewer>
</UserControl>
