// Copyright Epic Games, Inc. All Rights Reserved.

using Avalonia.Controls;

namespace UnrealToolbox.Plugins.HordeAgent
{
	partial class HordeAgentSettingsPage : UserControl
	{
		public HordeAgentSettingsPage()
			: this(SettingsContext.Default, null)
		{
		}

		public HordeAgentSettingsPage(SettingsContext context, HordeAgentPlugin? plugin)
		{
			InitializeComponent();

			DataContext = new HordeAgentSettingsViewModel(context, plugin);
			
			// Multiply with two as ProcessorCount may not return all logical processors depending on system
			// TODO: Replace with more accurate lookup (requires more .NET deps)
			CpuCount.Maximum = Environment.ProcessorCount * 2;
		}
	}
}