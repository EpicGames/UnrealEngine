// Copyright Epic Games, Inc. All Rights Reserved.

using System.ComponentModel;
using System.Diagnostics;
using Avalonia.Platform.Storage;
using CommunityToolkit.Mvvm.ComponentModel;

namespace UnrealToolbox.Plugins.HordeAgent
{
	partial class HordeAgentSettingsViewModel : ObservableObject
	{
		readonly SettingsContext _context;
		readonly HordeAgentPlugin? _plugin;

		AgentMode _mode;

		static string GetPropertyName(AgentMode mode)
			=> mode switch
			{
				AgentMode.Dedicated => nameof(IsDedicatedMode),
				AgentMode.Workstation => nameof(IsWorkstationMode),
				_ => nameof(IsDisabled)
			};

		void SetModeIfTrue(AgentMode mode, bool value)
		{
			if (value && mode != _mode)
			{
				string oldName = GetPropertyName(_mode);
				string newName = GetPropertyName(mode);

				OnPropertyChanging(oldName);
				OnPropertyChanging(newName);

				_mode = mode;

				OnPropertyChanged(oldName);
				OnPropertyChanged(newName);
			}
		}

		public bool IsDedicatedMode
		{
			get => _mode == AgentMode.Dedicated;
			set => SetModeIfTrue(AgentMode.Dedicated, value);
		}

		public bool IsWorkstationMode
		{
			get => _mode == AgentMode.Workstation;
			set => SetModeIfTrue(AgentMode.Workstation, value);
		}

		public bool IsDisabled
		{
			get => _mode == AgentMode.Disabled;
			set => SetModeIfTrue(AgentMode.Disabled, value);
		}

		[ObservableProperty]
		int _minIdleTimeSecs;

		[ObservableProperty]
		int _minFreeMemoryMb;

		[ObservableProperty]
		int _minFreeCpuPct;
		
		[ObservableProperty]
		int _cpuCount;
		
		[ObservableProperty]
		double _cpuMultiplier;

		[ObservableProperty]
		string _sandboxDir = "C:\\Dump"; // TODO

		public HordeAgentSettingsViewModel(SettingsContext context, HordeAgentPlugin? plugin)
		{
			_context = context;
			_plugin = plugin;

			HordeAgentSettings settings = plugin?.Settings ?? new HordeAgentSettings();
			_mode = settings.Mode ?? AgentMode.Disabled;
			_minIdleTimeSecs = settings.Idle.MinIdleTimeSecs;
			_minFreeMemoryMb = settings.Idle.MinFreeVirtualMemMb;
			_minFreeCpuPct = settings.Idle.MinIdleCpuPct;
			_cpuCount = settings.Cpu.CpuCount;
			_cpuMultiplier = settings.Cpu.CpuMultiplier;
		}

		protected override void OnPropertyChanged(PropertyChangedEventArgs e)
		{
			base.OnPropertyChanged(e);

			if (_plugin != null)
			{
				HordeAgentSettings settings = new HordeAgentSettings();

				settings.Mode = _mode;
				settings.Idle.MinIdleTimeSecs = MinIdleTimeSecs;
				settings.Idle.MinFreeVirtualMemMb = MinFreeMemoryMb;
				settings.Idle.MinIdleCpuPct = MinFreeCpuPct;
				settings.Cpu.CpuCount = CpuCount;
				settings.Cpu.CpuMultiplier = CpuMultiplier;

				_plugin.UpdateSettings(settings);
			}
		}

		public void OpenSandbox()
		{
			ProcessStartInfo startInfo = new ProcessStartInfo("explorer.exe");
			startInfo.ArgumentList.Add(SandboxDir);
			Process.Start(startInfo);
		}

		public async Task MoveSandboxAsync()
		{
			if (_context.SettingsWindow != null)
			{
				IStorageProvider storageProvider = _context.SettingsWindow.StorageProvider;

				FolderPickerOpenOptions options = new FolderPickerOpenOptions();
				options.AllowMultiple = false;
				options.SuggestedStartLocation = await storageProvider.TryGetFolderFromPathAsync(SandboxDir);

				IReadOnlyList<IStorageFolder> folders = await storageProvider.OpenFolderPickerAsync(options);
				if (folders.Count > 0)
				{
					string? path = folders[0].TryGetLocalPath();
					if (path != null)
					{
						SandboxDir = path;
					}
				}
			}
		}
	}
}
