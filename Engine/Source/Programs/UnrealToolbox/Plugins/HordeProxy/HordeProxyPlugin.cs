// Copyright Epic Games, Inc. All Rights Reserved.

using System.Net;
using Avalonia.Controls;
using EpicGames.Core;
using EpicGames.Horde;
using FluentAvalonia.UI.Controls;
using Microsoft.AspNetCore.Builder;
using Microsoft.AspNetCore.Hosting;
using Microsoft.AspNetCore.Http;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Hosting;

namespace UnrealToolbox.Plugins.HordeProxy
{
	class HordeProxyPlugin : ToolboxPluginBase
	{
		readonly BackgroundTask _serverTask;
		readonly IHordeClientProvider _hordeClientProvider;
		readonly AsyncEvent _restartServerEvent = new AsyncEvent();
		readonly SynchronizationContext? _synchronizationContext;
		readonly JsonConfig<HordeProxySettings> _settings;

		string _status = "Starting...";

		public HordeProxySettings Settings => _settings.Current;

		public event Action? OnStatusChanged;

		public override string Name => "Horde Proxy";

		public string Status => _status;

		public override IconSource Icon => new SymbolIconSource() { Symbol = Symbol.Filter };

		public HordeProxyPlugin(IHordeClientProvider hordeClientProvider)
		{
			_hordeClientProvider = hordeClientProvider;
			_hordeClientProvider.OnStateChanged += OnStateChanged;
			_synchronizationContext = SynchronizationContext.Current;

			DirectoryReference? settingsRoot = DirectoryReference.GetSpecialFolder(Environment.SpecialFolder.LocalApplicationData);
			settingsRoot ??= DirectoryReference.GetCurrentDirectory();
			_settings = new JsonConfig<HordeProxySettings>(FileReference.Combine(Program.DataDir, "HordeProxy.json"));
			_settings.LoadSettings();

			_serverTask = BackgroundTask.StartNew(RunServerAsync);
		}

		public override bool HasSettingsPage()
			=> true;

		public override Control CreateSettingsPage(SettingsContext context)
			=> new HordeProxySettingsPage(this);

		void SetStatus(string status)
		{
			_status = status;

			if (_synchronizationContext == null)
			{
				OnStatusChanged?.Invoke();
			}
			else
			{
				_synchronizationContext.Post(_ => OnStatusChanged?.Invoke(), null);
			}
		}

		public void UpdateSettings(HordeProxySettings settings)
		{
			if (_settings.UpdateSettings(settings))
			{
				_restartServerEvent.Pulse();
			}
		}

		public override async ValueTask DisposeAsync()
		{
			await base.DisposeAsync();

			_hordeClientProvider.OnStateChanged -= OnStateChanged;
			await _serverTask.DisposeAsync();
		}

		public override bool Refresh()
			=> false;

		void OnStateChanged()
		{
			_restartServerEvent.Pulse();
		}

		async Task RunServerAsync(CancellationToken cancellationToken)
		{
			while (!cancellationToken.IsCancellationRequested)
			{
				Task restartServerTask = _restartServerEvent.Task;

				HordeProxySettings settings = _settings.Current;
				if (settings.Enabled)
				{
					using CancellationTokenSource cancellationSource = CancellationTokenSource.CreateLinkedTokenSource(cancellationToken);
					Task stoppedTask = _restartServerEvent.Task.ContinueWith(_ => cancellationSource.Cancel(), cancellationSource.Token, TaskContinuationOptions.None, TaskScheduler.Default);

					try
					{
						SetStatus($"Running on http://localhost:{settings.Port}");

						WebApplicationBuilder builder = WebApplication.CreateBuilder();
						builder.Services.AddHttpClient();
						builder.Services.AddHorde();
						builder.WebHost.ConfigureKestrel(options => options.ListenLocalhost(settings.Port));

						await using WebApplication app = builder.Build();
						app.Map("/{*Route}", ForwardRequestAsync);
						await app.RunAsync(cancellationSource.Token);
					}
					catch (OperationCanceledException)
					{
						throw;
					}
					catch (Exception ex)
					{
						SetStatus($"Error: {ex.Message}");
					}

					await stoppedTask;
				}
				else
				{
					SetStatus("Stopped");
					await restartServerTask.WaitAsync(cancellationToken);
				}
				await Task.Delay(TimeSpan.FromSeconds(2.0), cancellationToken);
			}
		}

		static async Task ForwardRequestAsync(HttpContext context)
		{
			IHordeClient hordeClient = context.RequestServices.GetRequiredService<IHordeClient>();
			HttpClient httpClient = hordeClient.CreateHttpClient().HttpClient;

			if (context.Request.Method != "GET")
			{
				context.Response.StatusCode = (int)HttpStatusCode.MethodNotAllowed;
				return;
			}

			using HttpRequestMessage request = new HttpRequestMessage();
			request.Method = HttpMethod.Parse(context.Request.Method);
			request.RequestUri = new Uri(hordeClient.ServerUrl, $"{context.Request.Path}");
			request.Content = new StreamContent(context.Request.Body);

			using HttpResponseMessage response = await httpClient.SendAsync(request, context.RequestAborted);
			context.Response.StatusCode = (int)response.StatusCode;
			context.Response.ContentType = response.Content.Headers.ContentType?.MediaType;
			context.Response.ContentLength = response.Content.Headers.ContentLength;

			using Stream stream = await response.Content.ReadAsStreamAsync(context.RequestAborted);
			await stream.CopyToAsync(context.Response.Body, context.RequestAborted);
		}
	}
}
