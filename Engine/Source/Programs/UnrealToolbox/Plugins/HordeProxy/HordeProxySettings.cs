// Copyright Epic Games, Inc. All Rights Reserved.

#pragma warning disable CS1591

namespace UnrealToolbox.Plugins.HordeProxy
{
	[Serializable]
	public class HordeProxySettings
	{
		public bool Enabled { get; set; }
		public int Port { get; set; } = 13344;
	}
}
