<UserControl xmlns="https://github.com/avaloniaui"
		xmlns:x="http://schemas.microsoft.com/winfx/2006/xaml"
		xmlns:d="http://schemas.microsoft.com/expression/blend/2008"
		xmlns:mc="http://schemas.openxmlformats.org/markup-compatibility/2006"
		xmlns:local="using:UnrealToolbox.Plugins.HordeProxy"
		mc:Ignorable="d" d:DesignWidth="800" d:DesignHeight="1000"
		x:Class="UnrealToolbox.Plugins.HordeProxy.HordeProxySettingsPage"
		x:DataType="local:HordeProxySettingsViewModel"
		Background="{DynamicResource SolidBackgroundFillColorBaseBrush}">

	<UserControl.Styles>
		<StyleInclude Source="/Styles.axaml"/>
	</UserControl.Styles>

	<ScrollViewer>
		<StackPanel Margin="20" Spacing="5">

			<TextBlock Theme="{StaticResource SubtitleTextBlockStyle}" Margin="0 0 0 0">Horde Proxy</TextBlock>

			<TextBlock Classes="SettingHeader">General</TextBlock>

			<Border Classes="Card" Padding="16 6 16 2">
				<StackPanel Spacing="10" Margin="0 0 0 5">
					<Grid ColumnDefinitions="*,Auto">
						<StackPanel Grid.Column="0" Margin="0 10 20 10">
							<TextBlock Classes="SettingCaption">Enable Proxy Server</TextBlock>
							<TextBlock Classes="SettingHelp">Provides an unauthenticated connection to via localhost. Useful for scripting or accessing Horde's symbol server through Visual Studio without applications having to handle authentication.</TextBlock>
						</StackPanel>
						<ToggleSwitch Grid.Column="1" VerticalAlignment="Center" OnContent="" OffContent="" IsChecked="{Binding Enabled}"/>
					</Grid>
				</StackPanel>
			</Border>

			<TextBlock Classes="SettingHeader" IsVisible="{Binding Enabled}">Network</TextBlock>

			<Border Classes="Card" IsVisible="{Binding Enabled}">
				<Grid ColumnDefinitions="*,Auto">
					<StackPanel Grid.Column="0">
						<TextBlock Classes="SettingCaption">Port</TextBlock>
						<TextBlock Classes="SettingHelp">Local port number to allow connecting to the Horde Server</TextBlock>
					</StackPanel>
					<NumericUpDown VerticalAlignment="Center" Grid.Column="1" ShowButtonSpinner="False" Minimum="1" Maximum="65536" FormatString="0" TextAlignment="Right" Width="150" Value="{Binding Port}"/>
				</Grid>
			</Border>

			<TextBlock Classes="SettingHeader" IsVisible="{Binding Enabled}">Status</TextBlock>

			<Border Classes="Card" IsVisible="{Binding Enabled}">
				<Grid ColumnDefinitions="*,Auto">
					<TextBlock Classes="SettingHelp" Text="{Binding Status}"/>
				</Grid>
			</Border>

		</StackPanel>
	</ScrollViewer>
</UserControl>
