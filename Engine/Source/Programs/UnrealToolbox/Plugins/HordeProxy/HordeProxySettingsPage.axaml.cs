// Copyright Epic Games, Inc. All Rights Reserved.

using Avalonia.Controls;

namespace UnrealToolbox.Plugins.HordeProxy
{
	partial class HordeProxySettingsPage : UserControl
	{
		public HordeProxySettingsPage()
			: this(null)
		{
		}

		public HordeProxySettingsPage(HordeProxyPlugin? plugin)
		{
			InitializeComponent();

			DataContext = new HordeProxySettingsViewModel(plugin);
		}
	}
}