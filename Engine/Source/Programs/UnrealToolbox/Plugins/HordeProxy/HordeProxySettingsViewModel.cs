// Copyright Epic Games, Inc. All Rights Reserved.

using System.ComponentModel;
using CommunityToolkit.Mvvm.ComponentModel;

namespace UnrealToolbox.Plugins.HordeProxy
{
	partial class HordeProxySettingsViewModel : ObservableObject
	{
		readonly HordeProxyPlugin? _plugin;

		int _pendingUpdate;

		[ObservableProperty]
		bool _enabled;

		[ObservableProperty]
		int _port;

		[ObservableProperty]
		string _status = String.Empty;

		public HordeProxySettingsViewModel(HordeProxyPlugin? plugin)
		{
			_plugin = plugin;

			HordeProxySettings settings = plugin?.Settings ?? new HordeProxySettings();
			_enabled = settings.Enabled;
			_port = settings.Port;

			if (_plugin != null)
			{
				_status = _plugin.Status;
				_plugin.OnStatusChanged += OnStatusChanged;
			}
		}

		void OnStatusChanged()
			=> Status = _plugin?.Status ?? String.Empty;

		protected override void OnPropertyChanged(PropertyChangedEventArgs e)
		{
			base.OnPropertyChanged(e);

			if (_plugin != null)
			{
				int currentUpdate = Interlocked.Increment(ref _pendingUpdate);
				ApplySettingsAsync(currentUpdate);
			}
		}

		async void ApplySettingsAsync(int currentUpdate)
		{
			await Task.Delay(TimeSpan.FromSeconds(1.0));

			if (Interlocked.CompareExchange(ref _pendingUpdate, 0, 0) == currentUpdate)
			{
				ApplySettings();
			}
		}

		void ApplySettings()
		{
			if (_plugin != null)
			{
				HordeProxySettings settings = new HordeProxySettings();
				settings.Enabled = Enabled;
				settings.Port = Port;
				_plugin.UpdateSettings(settings);
			}
		}
	}
}
