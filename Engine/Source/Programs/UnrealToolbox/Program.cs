// Copyright Epic Games, Inc. All Rights Reserved.

using System.Runtime.InteropServices;
using Avalonia;
using Avalonia.Controls;
using EpicGames.Core;

namespace UnrealToolbox
{
	static class Program
	{
		const string MutexName = "UnrealToolbox-Mutex";
		public const string CloseEventName = "UnrealToolbox-Close";
		public const string RefreshEventName = "UnrealToolbox-Refresh"; // Note: this event is set by InstallerCustomActions when agent is installed.
		public const string SettingsEventName = "UnrealToolbox-Settings";
		public const string CommandEventName = "UnrealToolbox-Command";

		public static SelfUpdateState? Update { get; private set; }

		public static DirectoryReference DataDir { get; } = GetDataDir();

		public static FileReference LogFile { get; } = FileReference.Combine(DataDir, "Log.txt");

		[STAThread]
		public static int Main(string[] args)
		{
			if (!args.Any(x => x.Equals("-NoUpdate", StringComparison.OrdinalIgnoreCase)))
			{
				Update = SelfUpdateState.TryCreate("Unreal Toolbox", args);
			}

			for (; ; )
			{
				if (Update != null && Update.TryLaunchLatest())
				{
					return 0;
				}

				int result = RealMain(args);
				if (Update == null || !Update.IsUpdatePending())
				{
					return result;
				}
			}
		}

		static int RealMain(string[] args)
		{
			using EventWaitHandle? closeEvent = CreateNamedEvent(CloseEventName);
			if (args.Any(x => x.Equals("-Close", StringComparison.OrdinalIgnoreCase)))
			{
				closeEvent?.Set();
				return 0;
			}

			using EventWaitHandle? refreshEvent = CreateNamedEvent(RefreshEventName);
			if (args.Any(x => x.Equals("-Refresh", StringComparison.OrdinalIgnoreCase)))
			{
				refreshEvent?.Set();
				return 0;
			}

			using EventWaitHandle? settingsEvent = CreateNamedEvent(SettingsEventName);
			if (!args.Any(x => x.Equals("-Quiet", StringComparison.OrdinalIgnoreCase)))
			{
				settingsEvent?.Set();
			}

			using EventWaitHandle commandEvent = new EventWaitHandle(false, EventResetMode.AutoReset, CommandEventName);
			foreach (string arg in args)
			{
				const string UrlArgument = "-Url=";
				if (arg.StartsWith(UrlArgument, StringComparison.OrdinalIgnoreCase))
				{
					CommandHelper.Enqueue(new Command { Type = CommandType.OpenUrl, Argument = arg.Substring(UrlArgument.Length) });
					commandEvent.Set();
				}
			}

			using SingleInstanceMutex mutex = new SingleInstanceMutex(MutexName);
			if (!mutex.Wait(0))
			{
				return 1;
			}

			Log.AddFileWriter("Default", LogFile);

			AppBuilder builder = BuildAvaloniaApp();
			try
			{
				builder.StartWithClassicDesktopLifetime(args, ShutdownMode.OnExplicitShutdown);
			}
			finally
			{
				if (builder.Instance is App app)
				{
					Task.Run(async () => await app.DisposeAsync()).Wait();
				}
			}

			return 0;
		}

		// Avalonia configuration, don't remove; also used by visual designer.
		public static AppBuilder BuildAvaloniaApp()
		{
			return AppBuilder.Configure<App>()
				.UsePlatformDetect()
				.LogToTrace();
		}

		static DirectoryReference GetDataDir()
		{
			DirectoryReference? appDataDir = DirectoryReference.GetSpecialFolder(Environment.SpecialFolder.LocalApplicationData);
			if (appDataDir == null)
			{
				return DirectoryReference.GetCurrentDirectory();
			}
			else
			{
				return DirectoryReference.Combine(appDataDir, "Epic Games", "Unreal Toolbox");
			}
		}

		static EventWaitHandle? CreateNamedEvent(string name)
		{
			if (OperatingSystem.IsWindows())
			{
				return new EventWaitHandle(false, EventResetMode.AutoReset, name);
			}
			else
			{
				return null; // Need to figure out a way of replicating this functionality on MacOS/Linux.
			}
		}

		[DllImport("user32.dll")]
		static extern uint SetForegroundWindow(nint hWnd);

		public static void BringWindowToFront(Window window)
		{
			window.WindowState &= ~WindowState.Minimized;
			window.BringIntoView();

			window.Activate();
			window.Show();

			if (OperatingSystem.IsWindows())
			{
				// Avalonia doesn't seem to activate the window despite the call above, so fall back to pinvoke
				nint? handle = window.TryGetPlatformHandle()?.Handle;
				if (handle.HasValue)
				{
#pragma warning disable CA1806
					SetForegroundWindow(handle.Value);
#pragma warning restore CA1806
				}
			}
		}
	}

	class SingleInstanceMutex : IDisposable
	{
		readonly Mutex _mutex;
		bool _locked;

		public SingleInstanceMutex(string name)
		{
			_mutex = new Mutex(false, name);
		}

		public void Release()
		{
			if (_locked)
			{
				_mutex.ReleaseMutex();
				_locked = false;
			}
		}

		public bool Wait(int timeout)
		{
			if (!_locked)
			{
				try
				{
					_locked = _mutex.WaitOne(timeout);
				}
				catch (AbandonedMutexException)
				{
					_locked = true;
				}
			}
			return _locked;
		}

		public void Dispose()
		{
			Release();
			_mutex.Dispose();
		}
	}
}
