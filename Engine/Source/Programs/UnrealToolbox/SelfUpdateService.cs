// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using EpicGames.Horde.Storage.Nodes;
using EpicGames.Horde.Tools;
using Microsoft.Extensions.Logging;
using System.Diagnostics;
using System.Diagnostics.CodeAnalysis;
using System.Reflection;
using System.Text.Json;

namespace UnrealToolbox
{
	class SelfUpdateState
	{
		class JsonState
		{
			public string? LaunchApp { get; set; }
			public string? LatestVersion { get; set; }
			public string? UpdateVersion { get; set; }
		}

		readonly string? _currentVersion;
		readonly DirectoryReference _baseDir;
		readonly JsonConfig<JsonState> _config;
		readonly string[] _args;
		readonly bool _isLaunchApp;

		public string? CurrentVersionString
			=> _currentVersion;

		public string? LatestVersion
			=> _config.Current.LatestVersion;

		public DirectoryReference LatestDir
			=> DirectoryReference.Combine(_baseDir, "Latest");

		public DirectoryReference UpdateDir
			=> DirectoryReference.Combine(_baseDir, "Update");

		public string? UpdateVersion
		{
			get => _config.Current.UpdateVersion;
			set => UpdateState(x => x.UpdateVersion = value);
		}

		public SelfUpdateState(DirectoryReference baseDir, string[] args)
		{
			_baseDir = baseDir;
			_config = new JsonConfig<JsonState>(FileReference.Combine(baseDir, "Update.json"));
			_config.LoadSettings();
			_args = args;

			FileReference currentAssembly = new FileReference(Assembly.GetExecutingAssembly().Location);
			_currentVersion = ReadVersion(currentAssembly.Directory);

			_isLaunchApp = !currentAssembly.IsUnderDirectory(_baseDir);
			if (_isLaunchApp && !String.Equals(_config.Current.LaunchApp, currentAssembly.FullName, StringComparison.OrdinalIgnoreCase))
			{
				UpdateState(x => x.LaunchApp = currentAssembly.FullName);
			}
		}

		public bool IsUpdatePending()
			=> !String.IsNullOrEmpty(_config.Current.UpdateVersion);

		void UpdateState(Action<JsonState> update)
		{
			JsonState curState = _config.Current;

			JsonState newState = new JsonState();
			newState.LaunchApp = curState.LaunchApp;
			newState.LatestVersion = curState.LatestVersion;
			newState.UpdateVersion = curState.UpdateVersion;
			update(newState);

			_config.UpdateSettings(newState);
		}

		public static SelfUpdateState? TryCreate(string appName, string[] args)
		{
			DirectoryReference? localAppData = DirectoryReference.GetSpecialFolder(Environment.SpecialFolder.LocalApplicationData);
			if (localAppData == null)
			{
				return null;
			}

			DirectoryReference baseDir = DirectoryReference.Combine(localAppData, "Epic Games", appName);
			return new SelfUpdateState(baseDir, args);
		}

		public bool TryLaunchLatest()
		{
			if (_isLaunchApp)
			{
				string launchApp = Assembly.GetExecutingAssembly().Location;
				if (!String.Equals(_config.Current.LaunchApp, launchApp, StringComparison.OrdinalIgnoreCase))
				{
					UpdateState(x => x.LaunchApp = launchApp);
				}

				string? updateVersion = _config.Current.UpdateVersion;
				if (!String.IsNullOrEmpty(updateVersion))
				{
					if (!String.IsNullOrEmpty(_config.Current.LatestVersion))
					{
						UpdateState(x => x.LatestVersion = null);
						FileUtils.ForceDeleteDirectory(LatestDir);
					}

					UpdateState(x => x.UpdateVersion = null);
					DirectoryReference.Move(UpdateDir, LatestDir);
					UpdateState(x => x.LatestVersion = updateVersion);
				}

				if (!String.IsNullOrEmpty(_config.Current.LatestVersion))
				{
					string fileName = Path.GetFileName(Assembly.GetExecutingAssembly().Location);
					FileReference executable = FileReference.Combine(LatestDir, fileName);
					if (FileReference.Exists(executable) && ShouldLaunchLatest(LatestDir))
					{
						return Launch(executable.FullName, _args);
					}
				}
			}
			else
			{
				string? updateVersion = _config.Current.UpdateVersion;
				if (!String.IsNullOrEmpty(updateVersion) && !String.IsNullOrEmpty(_config.Current.LaunchApp))
				{
					return Launch(_config.Current.LaunchApp, _args);
				}
			}
			return false;
		}

		static bool TryParseVersion(string? version, [NotNullWhen(true)] out VersionNumber? versionNumber)
		{
			if (version == null)
			{
				versionNumber = null;
				return false;
			}

			version = version.Replace("-PF-", ".", StringComparison.OrdinalIgnoreCase);
			version = version.Replace("-", ".", StringComparison.Ordinal);

			return VersionNumber.TryParse(version, out versionNumber);
		}

		bool ShouldLaunchLatest(DirectoryReference latestDir)
			=> ShouldUpdateTo(ReadVersion(latestDir));

		/// <summary>
		/// Test whether the current instance should update to the given version
		/// </summary>
		public bool ShouldUpdateTo(string? version)
		{
			// If this build is not versioned, do not upgrade
			VersionNumber? currentVersionNumber;
			if (!TryParseVersion(_currentVersion, out currentVersionNumber))
			{
				return false;
			}

			// If the other build is not versioned, do not upgrade
			VersionNumber? latestVersionNumber;
			if (!TryParseVersion(version, out latestVersionNumber))
			{
				return false;
			}

			// Otherwise, only upgrade if the latest version is newer
			return latestVersionNumber > currentVersionNumber;
		}

		class VersionFile
		{
			public string? Version { get; set; }
		}

		static string? ReadVersion(DirectoryReference dir)
		{
			FileReference file = FileReference.Combine(dir, "Version.json");
			if (!FileReference.Exists(file))
			{
				return null;
			}

			string? version;
			try
			{
				byte[] versionData = FileReference.ReadAllBytes(file);
				VersionFile? versionFile = JsonSerializer.Deserialize<VersionFile>(versionData, new JsonSerializerOptions { PropertyNameCaseInsensitive = true });
				version = versionFile?.Version;
			}
			catch
			{
				return null;
			}

			return version;
		}

		static bool Launch(string executable, string[] args)
		{
			using Process process = new Process();
			if (executable.EndsWith(".dll", StringComparison.OrdinalIgnoreCase))
			{
				string baseExecutable = executable.Substring(0, executable.Length - 4);
				if (OperatingSystem.IsWindows())
				{
					baseExecutable += ".exe";
				}

				if (File.Exists(baseExecutable))
				{
					process.StartInfo.FileName = baseExecutable;
				}
				else
				{
					process.StartInfo.FileName = "dotnet";
					process.StartInfo.ArgumentList.Add(executable);
				}
			}
			else
			{
				process.StartInfo.FileName = executable;
			}

			foreach (string arg in args)
			{
				process.StartInfo.ArgumentList.Add(arg);
			}

			return process.Start();
		}
	}

	class SelfUpdateService : IAsyncDisposable
	{
		static ToolId ToolId { get; } = new ToolId("unreal-toolbox");

		readonly IHordeClientProvider _hordeClientProvider;
		readonly BackgroundTask _backgroundTask;
		readonly ILogger _logger;

		public event Action? OnUpdateReady;

		public SelfUpdateService(IHordeClientProvider hordeClientProvider, ILogger<SelfUpdateService> logger)
		{
			_hordeClientProvider = hordeClientProvider;
			_backgroundTask = new BackgroundTask(CheckForUpdatesAsync);
			_logger = logger;
		}

		public void Start()
		{
			_backgroundTask.Start();
		}

		public async ValueTask DisposeAsync()
		{
			await _backgroundTask.DisposeAsync();
		}

		async Task CheckForUpdatesAsync(CancellationToken cancellationToken)
		{
			for (; ; )
			{
				try
				{
					await CheckForUpdateAsync(cancellationToken);
				}
				catch (OperationCanceledException)
				{
					throw;
				}
				catch (Exception ex)
				{
					_logger.LogError(ex, "Error checking for updates: {Message}", ex.Message);
				}
				await Task.Delay(TimeSpan.FromMinutes(10.0), cancellationToken);
			}
		}

		async Task CheckForUpdateAsync(CancellationToken cancellationToken)
		{
			SelfUpdateState? selfUpdate = Program.Update;
			if (selfUpdate == null)
			{
				return;
			}

			using IHordeClientRef? clientRef = _hordeClientProvider.GetClientRef();
			if (clientRef == null)
			{
				return;
			}

			ITool? tool = await clientRef.Client.Tools.GetAsync(ToolId, cancellationToken);
			if (tool == null || tool.Deployments.Count == 0)
			{
				return;
			}

			IToolDeployment deployment = tool.Deployments[^1];
			_logger.LogInformation("Latest deployment is {Id} ({Version})", deployment.Id, deployment.Version);

			string updateVersion = deployment.Id.ToString();
			if (!String.Equals(updateVersion, selfUpdate.LatestVersion, StringComparison.OrdinalIgnoreCase) 
				&& !String.Equals(updateVersion, selfUpdate.UpdateVersion, StringComparison.OrdinalIgnoreCase)
				&& selfUpdate.ShouldUpdateTo(deployment.Version))
			{
				selfUpdate.UpdateVersion = null;

				DirectoryReference updateDir = selfUpdate.UpdateDir;
				DirectoryReference.CreateDirectory(updateDir);

				FileUtils.ForceDeleteDirectoryContents(updateDir);

				await deployment.Content.ExtractAsync(updateDir.ToDirectoryInfo(), new ExtractOptions(), _logger, cancellationToken);

				selfUpdate.UpdateVersion = updateVersion;
				_logger.LogInformation("Extracted {UpdateVersion} to {UpdateDir}", updateVersion, updateDir);
			}

			if (selfUpdate.IsUpdatePending())
			{
				_logger.LogInformation("Triggering update to {UpdateVersion}", selfUpdate.UpdateVersion);
				OnUpdateReady?.Invoke();
			}
		}
	}
}
