<Window xmlns="https://github.com/avaloniaui"
		xmlns:x="http://schemas.microsoft.com/winfx/2006/xaml"
		xmlns:d="http://schemas.microsoft.com/expression/blend/2008"
		xmlns:mc="http://schemas.openxmlformats.org/markup-compatibility/2006"
		xmlns:pages="using:UnrealToolbox"
		xmlns:ui="using:FluentAvalonia.UI.Controls"
		mc:Ignorable="d" d:DesignWidth="800" d:DesignHeight="450"
		x:Class="UnrealToolbox.SettingsWindow"
		Width="1000"
		Height="600"
		WindowStartupLocation="CenterScreen" 
		ExtendClientAreaChromeHints="NoChrome"
		Title="Settings"
		Icon="/Resources/StatusNormal.ico">

	<Panel>
		<ui:NavigationView x:Name="_navView" OpenPaneLength="260" IsSettingsVisible="False" PaneDisplayMode="Left" PaneTitle="Unreal Toolbox" BorderBrush="Transparent">
		</ui:NavigationView>
	</Panel>

</Window>
