// Copyright Epic Games, Inc. All Rights Reserved.

using Avalonia.Controls;
using FluentAvalonia.UI.Controls;
using Microsoft.Extensions.DependencyInjection;

namespace UnrealToolbox
{
	record class SettingsContext(Window? SettingsWindow, IServiceProvider ServiceProvider)
	{
		public static SettingsContext Default { get; } = new SettingsContext(null!, new ServiceCollection().BuildServiceProvider());
	}

	partial class SettingsWindow : Window
	{
		readonly IServiceProvider? _serviceProvider;
		readonly Dictionary<string, IToolboxPlugin> _typeToPlugin = new Dictionary<string, IToolboxPlugin>();
		readonly Dictionary<IToolboxPlugin, Control> _pluginPages = new Dictionary<IToolboxPlugin, Control>();

		AboutPage? _aboutPage;
		GeneralSettingsPage? _generalSettingsPage;

		public SettingsWindow()
			: this(null!)
		{ }

		public SettingsWindow(IServiceProvider serviceProvider)
		{
			InitializeComponent();

			_serviceProvider = serviceProvider;
			_navView.SelectionChanged += NavView_SelectionChanged;

			Activated += OnActivated;

			Refresh();
		}

		public void Refresh()
		{
			if (_serviceProvider != null)
			{
				object? selectedItemContent = (_navView.SelectedItem as NavigationViewItem)?.Content;

				_typeToPlugin.Clear();

				_navView.MenuItems.Clear();
				_navView.MenuItems.Add(new NavigationViewItem() { Content = "General", IconSource = new SymbolIconSource() { Symbol = Symbol.Settings }, Tag = typeof(GeneralSettingsPage).FullName });

				foreach (IToolboxPlugin plugin in _serviceProvider.GetServices<IToolboxPlugin>())
				{
					if (plugin.HasSettingsPage())
					{
						string typeName = plugin.GetType().FullName!;
						_navView.MenuItems.Add(new NavigationViewItem() { Content = plugin.Name, IconSource = plugin.Icon, Tag = typeName });
						_typeToPlugin.Add(typeName, plugin);
					}
				}

				_navView.FooterMenuItems.Clear();
				_navView.FooterMenuItems.Add(new NavigationViewItem() { Content = "About", IconSource = new SymbolIconSource() { Symbol = Symbol.Help } });

				_navView.SelectedItem =
					_navView.MenuItems.FirstOrDefault(x => Object.Equals((x as NavigationViewItem)?.Content, selectedItemContent))
					?? _navView.FooterMenuItems.FirstOrDefault(x => Object.Equals((x as NavigationViewItem)?.Content, selectedItemContent))
					?? _navView.MenuItems[0];

				NavView_UpdateContent();
			}
		}

		void OnActivated(object? sender, EventArgs e)
		{
			if (_serviceProvider != null)
			{
				bool updated = false;
				foreach (IToolboxPlugin plugin in _serviceProvider.GetServices<IToolboxPlugin>())
				{
					updated |= plugin.Refresh();
				}
				if (updated)
				{
					Refresh();
				}
			}
		}

		private void NavView_SelectionChanged(object? sender, NavigationViewSelectionChangedEventArgs e)
		{
			if (sender == _navView)
			{
				NavView_UpdateContent();
			}
		}

		private void NavView_UpdateContent()
		{
			if (_navView.SelectedItem is NavigationViewItem nvi)
			{
				SettingsContext context = new SettingsContext(this, _serviceProvider!);

				string? typeName = (string?)nvi.Tag;
				if (typeName != null && _typeToPlugin.TryGetValue(typeName, out IToolboxPlugin? plugin))
				{
					Control pluginPage = CreatePluginPage(plugin, context);
					_navView.Content = pluginPage;
				}
				else if (nvi == _navView.FooterMenuItems.FirstOrDefault())
				{
					_aboutPage ??= new AboutPage();
					_navView.Content = _aboutPage;
				}
				else
				{
					_generalSettingsPage ??= new GeneralSettingsPage(context);
					_navView.Content = _generalSettingsPage;
				}
			}
		}

		private Control CreatePluginPage(IToolboxPlugin plugin, SettingsContext context)
		{
			Control? pluginPage;
			if (!_pluginPages.TryGetValue(plugin, out pluginPage))
			{
				pluginPage = plugin.CreateSettingsPage(context);
				_pluginPages.Add(plugin, pluginPage);
			}
			return pluginPage;
		}
	}
}
