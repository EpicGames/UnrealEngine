<Styles xmlns="https://github.com/avaloniaui"
		xmlns:x="http://schemas.microsoft.com/winfx/2006/xaml">

	<Design.PreviewWith>
		<Panel Margin="50">
			<Border Classes="Card" Padding="20">
				<StackPanel>
					<TextBlock Classes="SettingCaption">This is a setting</TextBlock>
					<TextBlock Classes="SettingHelp">This is what the setting does</TextBlock>
					<TextBlock Classes="SettingSubHeading">This is what the setting does</TextBlock>
				</StackPanel>
			</Border>
		</Panel>
	</Design.PreviewWith>

	<Style Selector="Border.Card">
		<Setter Property="BorderBrush" Value="{DynamicResource CardStrokeColorDefaultBrush}"/>
		<Setter Property="BorderThickness" Value="1"/>
		<Setter Property="Background" Value="{DynamicResource CardBackgroundFillColorDefaultBrush}"/>
		<Setter Property="TextElement.Foreground" Value="{DynamicResource TextFillColorPrimaryBrush}"/>
		<Setter Property="CornerRadius" Value="{DynamicResource ControlCornerRadius}"/>
		<Setter Property="BackgroundSizing" Value="InnerBorderEdge"/>
		<Setter Property="Padding" Value="16 14"/>
	</Style>
	<Style Selector="TextBlock.SettingHeader">
		<Setter Property="Theme" Value="{DynamicResource BodyStrongTextBlockStyle}"/>
		<Setter Property="Margin" Value="0 16 0 8"/>
	</Style>
	<Style Selector="TextBlock.SettingCaption">
		<Setter Property="Theme" Value="{DynamicResource BodyTextBlockStyle}"/>
		<Setter Property="VerticalAlignment" Value="Center"/>		
	</Style>
	<Style Selector="TextBlock.SettingHelp">
		<Setter Property="Theme" Value="{DynamicResource CaptionTextBlockStyle}"/>
		<Setter Property="FontWeight" Value="Light"/>
		<Setter Property="VerticalAlignment" Value="Center"/>
	</Style>
	<Style Selector="TextBlock.SettingSubHeading">
		<Setter Property="Theme" Value="{DynamicResource CaptionTextBlockStyle}"/>
	</Style>

</Styles>
