// Copyright Epic Games, Inc. All Rights Reserved.

using System.Diagnostics;
using System.Runtime.InteropServices;
using System.Security.Authentication;
using System.Text.Json;
using EpicGames.Core;
using EpicGames.Horde;
using EpicGames.Horde.Storage.Nodes;
using EpicGames.Horde.Tools;
using Microsoft.Extensions.Logging;
using Microsoft.Win32;

#nullable enable

namespace UnrealToolbox
{
	class ToolCatalog : IToolCatalog, IAsyncDisposable
	{
		record class ItemState(string Name, string Description, string? MsiProductId, CurrentToolDeploymentInfo? Current, PendingToolDeploymentInfo? Pending, IToolDeployment? Latest);

		[DebuggerDisplay("{Id}")]
		class Item : IToolCatalogItem
		{
			public ToolId Id { get; }

			readonly ToolCatalog _catalog;

			// Keep a separate copy of the state on the main thread; we will update it explicitly. This prevents tearing of updates on UI threads.
			public ItemState _mainThreadState;
			public ToolDeploymentInfo? _mainThreadLatest;

			// Public properties only expose main thread data
			string IToolCatalogItem.Name => _mainThreadState.Name;
			string IToolCatalogItem.Description => _mainThreadState.Description;
			ToolDeploymentInfo? IToolCatalogItem.Latest => _mainThreadLatest;
			CurrentToolDeploymentInfo? IToolCatalogItem.Current => _mainThreadState.Current;
			PendingToolDeploymentInfo? IToolCatalogItem.Pending => _mainThreadState.Pending;

			public ItemState _state;
			public DeploymentTask? _deploymentTask;
			public event Action? OnItemChanged;

			public Item(ToolCatalog catalog, ToolId id, ItemState state)
			{
				_catalog = catalog;
				Id = id;
				_state = state;

				_mainThreadState = null!;
				UpdateMainThreadState();
			}

			public void UpdateMainThreadState()
			{
				_mainThreadState = _state;
				_mainThreadLatest = (_state.Latest == null) ? null : new ToolDeploymentInfo(_state.Latest.Id, _state.Latest.Version);
			}

			public void NotifyChanged()
			{
				OnItemChanged?.Invoke();
			}

			public void SetState(ItemState state)
			{
				_state = state;
				OnItemChanged?.Invoke();
			}

			public void Cancel()
				=> _catalog.CancelDeployment(this, _mainThreadState.Pending?.Deployment);

			// Installing or uninstalling changes the pending task state immediately, and anyone copying the pending state will always see the value of that state.
			// UI things may wait on that state, and will get notified from that state.
			// So we should basically post a notice to the main thread to say we've queued up the next state.

			public void Install()
				=> _catalog.UpdateDeployment(this, _mainThreadState.Latest);

			public void Uninstall()
				=> _catalog.UpdateDeployment(this, null);
		}

#pragma warning disable CA1001 // Cancellation source is disposed by the task itself
		class DeploymentTask
#pragma warning restore CA1001
		{
			public ToolDeploymentInfo Deployment { get; }

			CancellationTokenSource? _cancellationSource;
			readonly Task _task;

			public DeploymentTask(ToolDeploymentInfo deployment, Func<CancellationToken, Task> func, DeploymentTask? lastDeploymentTask)
			{
				Deployment = deployment;

				CancellationTokenSource cancellationSource = new CancellationTokenSource();
				_task = Task.Run(() => RunAsync(func, lastDeploymentTask, cancellationSource), CancellationToken.None);

				_cancellationSource = cancellationSource;
			}

			async Task RunAsync(Func<CancellationToken, Task> func, DeploymentTask? lastDeploymentTask, CancellationTokenSource cancellationSource)
			{
				CancellationToken cancellationToken = cancellationSource!.Token;
				try
				{
					if (lastDeploymentTask != null)
					{
						lastDeploymentTask.Cancel();
						await lastDeploymentTask.WaitAsync(CancellationToken.None);
					}
					await func(cancellationToken);
				}
				catch (OperationCanceledException)
				{
				}
				finally
				{
					lock (_task)
					{
						cancellationSource.Dispose(); // Note; the instance passed into this function, not the class member.
						_cancellationSource = null;
					}
				}
			}

			public Task WaitAsync(CancellationToken cancellationToken)
				=> _task.WaitAsync(cancellationToken);

			public void Cancel()
			{
				lock (_task)
				{
					_cancellationSource?.Cancel();
					_cancellationSource = null;
				}
			}
		}

		#region Json serialization

		class JsonState
		{
			public bool AutoUpdate { get; set; }
			public List<JsonItemState> Items { get; set; } = new List<JsonItemState>();
		}

		class JsonItemState
		{
			public ToolId Id { get; set; }
			public string Name { get; set; }
			public string Description { get; set; }
			public string? MsiProductId { get; set; }
			public JsonDeploymentState? Current { get; set; }

			public JsonItemState()
			{
				Name = String.Empty;
				Description = String.Empty;
			}

			public JsonItemState(ToolId id, ItemState state)
			{
				Id = id;
				Name = state.Name;
				Description = state.Description;
				MsiProductId = state.MsiProductId;
				Current = (state.Current == null) ? null : new JsonDeploymentState(state.Current);
			}
		}

		class JsonDeploymentState
		{
			public ToolDeploymentId Id { get; set; }
			public string Version { get; set; }

			public JsonDeploymentState()
			{
				Version = String.Empty;
			}

			public JsonDeploymentState(ToolDeploymentInfo current)
			{
				Id = current.Id;
				Version = current.Version;
			}

			public ToolDeploymentInfo ToToolDeploymentInfo()
				=> new ToolDeploymentInfo(Id, Version);
		}

		#endregion

		readonly IHordeClientProvider _hordeClientProvider;
		readonly ILogger _logger;
		readonly object _workerThreadLockObject = new object();
		readonly AsyncEvent _updateEvent;
		readonly BackgroundTask _updateTask;
		readonly AsyncEvent _cleanupEvent;
		readonly BackgroundTask _cleanupTask;
		readonly SemaphoreSlim _msiSemaphore;

		readonly SynchronizationContext? _mainThreadSynchronizationContext;

		IReadOnlyDictionary<ToolId, Item> _mainThreadItems;
		IReadOnlyList<IToolCatalogItem> _mainThreadItemList;
		IReadOnlyList<IToolCatalogItem> IToolCatalog.Items => _mainThreadItemList;

		IReadOnlyDictionary<ToolId, Item> _items;

		Uri? _serverUri;
		bool _autoUpdate = true;

		public event Action? OnItemsChanged;

		static readonly JsonSerializerOptions s_jsonSerializerOptions = new JsonSerializerOptions { AllowTrailingCommas = true, PropertyNameCaseInsensitive = true, PropertyNamingPolicy = JsonNamingPolicy.CamelCase, WriteIndented = true };

		public bool AutoUpdate
		{
			get => _autoUpdate;
			set
			{
				if (_autoUpdate != value)
				{
					_autoUpdate = value;
					_ = SaveStateAsync(CancellationToken.None);
				}
			}
		}
		
		/// <inheritdoc/>
		public void RequestUpdate()
		{
			_updateEvent.Pulse();
		}

		public ToolCatalog(IHordeClientProvider hordeClientProvider, ILogger<ToolCatalog> logger)
		{
			_hordeClientProvider = hordeClientProvider;
			_logger = logger;
			_updateEvent = new AsyncEvent();
			_updateTask = new BackgroundTask(UpdateAsync);
			_cleanupEvent = new AsyncEvent();
			_cleanupTask = new BackgroundTask(CleanupAsync);
			_mainThreadSynchronizationContext = SynchronizationContext.Current;
			_msiSemaphore = new SemaphoreSlim(1);

			_items = LoadState();
			_mainThreadItems = null!;
			_mainThreadItemList = null!;

			UpdateMainThread();
		}
		
		public static DirectoryReference GetToolsDir() => DirectoryReference.Combine(Program.DataDir, "Tools");
		public static FileReference GetConfigFile() => FileReference.Combine(Program.DataDir, "Tools.json");

		void PostMainThreadUpdate()
		{
			if (_mainThreadSynchronizationContext == null)
			{
				UpdateMainThread();
			}
			else
			{
				_mainThreadSynchronizationContext.Post(_ => UpdateMainThread(), null);
			}
		}

		void UpdateMainThread()
		{
			bool updatedCollection = false;
			List<Item> updatedItems = new List<Item>();

			lock (_workerThreadLockObject)
			{
				if (_items != _mainThreadItems)
				{
					UpdateMainThreadItemsState();
					updatedCollection = true;
				}

				foreach (Item item in _items.Values)
				{
					if (item._state != item._mainThreadState)
					{
						item.UpdateMainThreadState();
						updatedItems.Add(item);
					}
				}
			}

			if (updatedCollection)
			{
				OnItemsChanged?.Invoke();
			}
			foreach (Item updatedItem in updatedItems)
			{
				updatedItem.NotifyChanged();
			}
		}

		void UpdateMainThreadItemsState()
		{
			_mainThreadItems = _items;
			_mainThreadItemList = _items.Values.OrderBy(x => x._mainThreadState.Name).ToArray();
		}

		public void Start()
		{
			_updateTask.Start();
			_cleanupTask.Start();

			_hordeClientProvider.OnStateChanged += OnServerChange;
		}

		public async Task StopAsync(CancellationToken cancellationToken)
		{
			_hordeClientProvider.OnStateChanged -= OnServerChange;

			foreach (Item item in _items.Values)
			{
				DeploymentTask? deploymentTask = item._deploymentTask;
				if (deploymentTask != null)
				{
					deploymentTask.Cancel();
					await deploymentTask.WaitAsync(cancellationToken);
				}
			}

			await _updateTask.StopAsync(cancellationToken);
			await _cleanupTask.StopAsync(cancellationToken);
		}

		public async ValueTask DisposeAsync()
		{
			await StopAsync(CancellationToken.None);
			await _updateTask.DisposeAsync();
			await _cleanupTask.DisposeAsync();
			_msiSemaphore.Dispose();
		}

		void OnServerChange()
		{
			_updateEvent.Pulse();
		}

		static DirectoryReference GetToolDir(ToolId toolId, ToolDeploymentId deploymentId)
		{
			return DirectoryReference.Combine(GetToolsDir(), toolId.ToString(), deploymentId.ToString());
		}

		ToolConfig LoadToolConfig(DirectoryReference toolDir)
		{
			FileReference configFile = FileReference.Combine(toolDir, "Toolbox.json");
			if (FileReference.Exists(configFile))
			{
				JsonSerializerOptions jsonSerializerOptions = new JsonSerializerOptions();
				jsonSerializerOptions.PropertyNameCaseInsensitive = true;
				jsonSerializerOptions.AllowTrailingCommas = true;
				jsonSerializerOptions.ReadCommentHandling = JsonCommentHandling.Skip;

				try
				{
					byte[] data = FileReference.ReadAllBytes(configFile);
					return JsonSerializer.Deserialize<ToolConfig>(data, jsonSerializerOptions) ?? new ToolConfig();
				}
				catch (Exception ex)
				{
					_logger.LogError(ex, "Error reading {File}: {Message}", configFile, ex.Message);
				}
			}
			return new ToolConfig();
		}

		IReadOnlyDictionary<ToolId, Item> LoadState()
		{
			Dictionary<ToolId, Item> items = new Dictionary<ToolId, Item>();

			byte[]? data = FileTransaction.ReadAllBytes(GetConfigFile());
			if (data != null)
			{
				JsonState? state = JsonSerializer.Deserialize<JsonState>(data, s_jsonSerializerOptions);
				if (state != null)
				{
					_autoUpdate = state.AutoUpdate;
					foreach (JsonItemState jsonItem in state.Items)
					{
						CurrentToolDeploymentInfo? current = null;
						if (jsonItem.Current != null)
						{
							DirectoryReference toolDir = GetToolDir(jsonItem.Id, jsonItem.Current.Id);
							ToolConfig toolConfig = LoadToolConfig(toolDir);
							current = new CurrentToolDeploymentInfo(jsonItem.Current.Id, jsonItem.Current.Version, toolDir, toolConfig);
						}

						ItemState itemState = new ItemState(jsonItem.Name, jsonItem.Description, jsonItem.MsiProductId, current, null, null);
						items.Add(jsonItem.Id, new Item(this, jsonItem.Id, itemState));
					}
				}
			}

			return items;
		}

		async Task SaveStateAsync(CancellationToken cancellationToken)
		{
			JsonState state;
			lock (_workerThreadLockObject)
			{
				state = new JsonState();
				state.AutoUpdate = _autoUpdate;
				foreach (Item item in _items.Values)
				{
					if (item._state.Current != null)
					{
						state.Items.Add(new JsonItemState(item.Id, item._state));
					}
				}
			}

			byte[] data = JsonSerializer.SerializeToUtf8Bytes(state, s_jsonSerializerOptions);

			DirectoryReference.CreateDirectory(GetConfigFile().Directory);
			await FileTransaction.WriteAllBytesAsync(GetConfigFile(), data, cancellationToken);
		}

		void CancelDeployment(Item item, ToolDeploymentInfo? deployment)
		{
			if (deployment != null)
			{
				lock (_workerThreadLockObject)
				{
					if (item._deploymentTask != null && item._deploymentTask.Deployment == deployment)
					{
						item._deploymentTask.Cancel();
					}
				}
			}
		}

		void UpdateDeployment(Item item, IToolDeployment? deployment)
		{
			lock (_workerThreadLockObject)
			{
				DeploymentTask? lastDeploymentTask = item._deploymentTask;
				if (deployment != null)
				{
					ToolDeploymentInfo deploymentInfo = new ToolDeploymentInfo(deployment.Id, deployment.Version);
					item._deploymentTask = new DeploymentTask(deploymentInfo, ctx => UpdateCurrentDeploymentAsync(item, deployment, deploymentInfo, ctx), lastDeploymentTask);
				}
				else if (item._state.Current != null)
				{
					ToolDeploymentInfo deploymentInfo = item._state.Current;
					item._deploymentTask = new DeploymentTask(deploymentInfo, ctx => UpdateCurrentDeploymentAsync(item, deployment, deploymentInfo, ctx), lastDeploymentTask);
				}
			}
		}

		async Task UpdateCurrentDeploymentAsync(Item item, IToolDeployment? deployment, ToolDeploymentInfo deploymentInfo, CancellationToken cancellationToken)
		{
			try
			{
				if (deployment == null)
				{
					await UninstallAsync(item, cancellationToken);
					_cleanupEvent.Set();
				}
				else
				{
					UpdateStateAndNotify(item, state => state with { Pending = new PendingToolDeploymentInfo(false, $"Updating to {deployment.Version}", deploymentInfo) });

					DirectoryReference toolDir = GetToolDir(item.Id, deploymentInfo.Id);
					DirectoryReference.CreateDirectory(toolDir);
					await deployment.Content.ExtractAsync(toolDir.ToDirectoryInfo(), new ExtractOptions(), _logger, cancellationToken);

					await InstallAsync(item, toolDir, deploymentInfo, cancellationToken);
				}

				await SaveStateAsync(cancellationToken);
			}
			catch (OperationCanceledException)
			{
				_logger.LogInformation("Deployment cancelled");
				UpdateStateAndNotify(item, state => state with { Pending = new PendingToolDeploymentInfo(true, $"Cancelled.", deploymentInfo) });
			}
			catch (Exception ex)
			{
				_logger.LogError(ex, "Error updating deployment: {Message}", ex.Message);
				UpdateStateAndNotify(item, state => state with { Pending = new PendingToolDeploymentInfo(true, $"Error updating. View Log for info.", deploymentInfo, ShowLogLink: true) });
			}
		}

		async Task InstallAsync(Item item, DirectoryReference toolDir, ToolDeploymentInfo deploymentInfo, CancellationToken cancellationToken)
		{
			ToolConfig toolConfig = LoadToolConfig(toolDir);

			if (!await UninstallInternalAsync(item, cancellationToken))
			{
				return;
			}

			if (toolConfig.InstallCommand != null)
			{
				PendingToolDeploymentInfo pending = new PendingToolDeploymentInfo(false, "Running install actions...", deploymentInfo);
				UpdateStateAndNotify(item, state => state with { Pending = pending });

				int exitCode = await RunCommandAsync(item.Id.ToString(), toolConfig.InstallCommand, toolDir.FullName, 
					toolConfig.RequiresElevation, toolConfig.Hidden, cancellationToken);
				if (exitCode != 0)
				{
					pending = new PendingToolDeploymentInfo(true, $"Installation failed ({exitCode})", deploymentInfo, ShowLogLink: true);
					UpdateStateAndNotify(item, state => state with { Pending = pending });
					return;
				}
			}
			else if (OperatingSystem.IsWindows() && !String.IsNullOrEmpty(item._state.MsiProductId))
			{
				// Wait to run the installer
				if (!await _msiSemaphore.WaitAsync(0, cancellationToken))
				{
					PendingToolDeploymentInfo pending = new PendingToolDeploymentInfo(false, "Waiting to install...", deploymentInfo);
					UpdateStateAndNotify(item, state => state with { Pending = pending });

					await _msiSemaphore.WaitAsync(cancellationToken);
				}

				// Keep the lock until the install has finished
				try
				{
					PendingToolDeploymentInfo pending = new PendingToolDeploymentInfo(false, "Running installer...", deploymentInfo);
					UpdateStateAndNotify(item, state => state with { Pending = pending });

					FileReference? msiFile = DirectoryReference.EnumerateFiles(toolDir, "*.msi").FirstOrDefault();
					if (msiFile == null)
					{
						pending = new PendingToolDeploymentInfo(true, $"MSI file not found", deploymentInfo);
						UpdateStateAndNotify(item, state => state with { Pending = pending });
						return;
					}

					ToolCommand installCommand = new ToolCommand { FileName = "msiexec.exe" };
					installCommand.Arguments = new List<string> { "/I", msiFile.FullName };

					int exitCode = await RunCommandAsync(item.Id.ToString(), installCommand, toolDir.FullName, 
						toolConfig.RequiresElevation, toolConfig.Hidden, cancellationToken);
					if (exitCode != 0 || !IsMsiInstalled(item._state.MsiProductId))
					{
						UpdateStateAndNotify(item, state => state with { Pending = null });
						return;
					}
				}
				finally
				{
					_msiSemaphore.Release();
				}
			}

			CurrentToolDeploymentInfo current = new CurrentToolDeploymentInfo(deploymentInfo.Id, deploymentInfo.Version, toolDir, toolConfig);
			UpdateStateAndNotify(item, state => state with { Pending = null, Current = current });
		}

		static bool IsMsiInstalled(string msiProductId)
		{
			if (!OperatingSystem.IsWindows())
			{
				return false;
			}

			string? value =
				(Registry.GetValue($"HKEY_LOCAL_MACHINE\\SOFTWARE\\Microsoft\\Windows\\CurrentVersion\\Uninstall\\{{{msiProductId}}}", "UninstallString", null) as string) ??
				(Registry.GetValue($"HKEY_LOCAL_MACHINE\\SOFTWARE\\WOW6432Node\\Microsoft\\Windows\\CurrentVersion\\Uninstall\\{{{msiProductId}}}", "UninstallString", null) as string);

			return !String.IsNullOrEmpty(value);
		}

		async Task<bool> UninstallAsync(Item item, CancellationToken cancellationToken)
		{
			if (!await UninstallInternalAsync(item, cancellationToken))
			{
				return false;
			}

			UpdateStateAndNotify(item, state => state with { Pending = null, Current = null });
			return true;
		}

		async Task<bool> UninstallInternalAsync(Item item, CancellationToken cancellationToken)
		{
			CurrentToolDeploymentInfo? current = item._state.Current;
			if (current != null)
			{
				if (current.Config.UninstallCommand != null)
				{
					PendingToolDeploymentInfo? pending = new PendingToolDeploymentInfo(false, $"Removing {current.Version}", null);
					UpdateStateAndNotify(item, state => state with { Pending = pending, Current = null });

					await RunCommandAsync(item.Id.ToString(), current.Config.UninstallCommand, current.Dir.FullName, 
						current.Config.RequiresElevation, current.Config.Hidden, cancellationToken);
				}
				else if (OperatingSystem.IsWindows() && !String.IsNullOrEmpty(item._state.MsiProductId) && IsMsiInstalled(item._state.MsiProductId))
				{
					PendingToolDeploymentInfo pending;

					// Wait to run the installer
					if (!await _msiSemaphore.WaitAsync(0, cancellationToken))
					{
						pending = new PendingToolDeploymentInfo(false, "Waiting to run installer...", current);
						UpdateStateAndNotify(item, state => state with { Pending = pending });

						await _msiSemaphore.WaitAsync(cancellationToken);
					}

					// Run the installer
					try
					{
						pending = new PendingToolDeploymentInfo(false, $"Running installer...", current);
						UpdateStateAndNotify(item, state => state with { Pending = pending });

						ToolCommand uninstallCommand = new ToolCommand { FileName = "msiexec.exe" };
						uninstallCommand.Arguments = new List<string> { "/x", $"{{{item._state.MsiProductId}}}" };

						int exitCode = await RunCommandAsync(item.Id.ToString(), uninstallCommand, Directory.GetCurrentDirectory(), 
							current.Config.RequiresElevation, current.Config.Hidden, cancellationToken);
						if (exitCode == 1602)
						{
							// User cancelled (https://learn.microsoft.com/en-us/windows/win32/msi/error-codes)
							UpdateStateAndNotify(item, state => state with { Pending = null });
							return false;
						}
						else if (exitCode != 0 || IsMsiInstalled(item._state.MsiProductId))
						{
							pending = new PendingToolDeploymentInfo(true, $"Uninstall failed ({exitCode}).", current);
							UpdateStateAndNotify(item, state => state with { Pending = pending });
							return false;
						}
					}
					finally
					{
						_msiSemaphore.Release();
					}
				}
			}
			return true;
		}

		async Task<int> RunCommandAsync(string toolName, ToolCommand command, string workingDir, bool elevateProcess, bool hidden, CancellationToken cancellationToken)
		{
			Dictionary<string, string> properties = new Dictionary<string, string>(StringComparer.OrdinalIgnoreCase);
			properties.Add("ToolDir", workingDir);

			string arguments = String.Empty;
			if (command.Arguments != null)
			{
				List<string> argumentList = command.Arguments.ConvertAll(x => StringUtils.ExpandProperties(x, properties));
				arguments = CommandLineArguments.Join(argumentList);
			}
			
			string fileName = Path.Combine(workingDir, StringUtils.ExpandProperties(command.FileName, properties));
			if (!File.Exists(fileName))
			{
				fileName = command.FileName;
			}
			
			_logger.LogInformation("{ToolName}> Running {FileName} {Arguments}", toolName, CommandLineArguments.Quote(fileName), arguments);
			
			using Process process = new Process();
			process.StartInfo = new ProcessStartInfo
			{
				FileName = fileName,
				Arguments = arguments,
				WorkingDirectory = workingDir,
				CreateNoWindow = true
			};
			
			if (hidden)
			{
				process.StartInfo.WindowStyle = ProcessWindowStyle.Hidden;
			}
			
			if (elevateProcess)
			{
				// When elevating the process, IO cannot be redirected in this process
				process.StartInfo.UseShellExecute = true;
				process.StartInfo.Verb = "runas"; // "runas" triggers the UAC prompt
				_logger.LogInformation("Tool install runs as an elevated process. To read process output, check installer's own logging mechanism.");
			}
			else
			{
				process.StartInfo.RedirectStandardOutput = true;
				process.StartInfo.RedirectStandardError = true;
				process.StartInfo.UseShellExecute = false;
				
				DataReceivedEventHandler onDataReceived = (sender, args) =>
				{
					if (args.Data != null)
					{
						_logger.LogInformation("{ToolName}> {Line}", toolName, args.Data);
					}
				};
				
				process.OutputDataReceived += onDataReceived;
				process.ErrorDataReceived += onDataReceived;
			}
			
			process.Start();
			if (!elevateProcess)
			{
				process.BeginOutputReadLine();
				process.BeginErrorReadLine();
			}
			
			await process.WaitForExitAsync(cancellationToken);
			int exitCode = process.ExitCode;
			_logger.LogInformation("{ToolName}> Exit code {ExitCode}", toolName, exitCode);
			
			return exitCode;
		}

		void UpdateStateAndNotify(Item item, Func<ItemState, ItemState> update)
		{
			lock (_workerThreadLockObject)
			{
				string prevState = GetItemStateMessage(item._state);
				item._state = update(item._state);

				string nextState = GetItemStateMessage(item._state);
				if (!String.Equals(prevState, nextState, StringComparison.Ordinal))
				{
					_logger.LogInformation("{ToolName}> {State}", item.Id, nextState);
				}
			}
			PostMainThreadUpdate();
		}

		static string GetItemStateMessage(ItemState state)
		{
			List<string> messages = new List<string>();

			if (state.Current != null)
			{
				messages.Add($"Current: {state.Current.Version} ({state.Current.Id})");
			}

			if (state.Pending != null)
			{
				if (state.Pending.Failed)
				{
					messages.Add($"Pending: {state.Pending.Message} (Failed)");
				}
				else if (state.Pending.Deployment != null)
				{
					messages.Add($"Pending: {state.Pending.Message} ({state.Pending.Deployment.Id}@{state.Pending.Deployment.Version})");
				}
			}

			if (messages.Count == 0)
			{
				messages.Add("Not installed");
			}

			return String.Join(", ", messages);
		}

		#region Cleanup

		async Task CleanupAsync(CancellationToken cancellationToken)
		{
			while (!cancellationToken.IsCancellationRequested)
			{
				Task cleanupTask = _cleanupEvent.Task;

				bool success = false;
				try
				{
					success = CleanupOnce();
				}
				catch (OperationCanceledException)
				{
					break;
				}
				catch (Exception ex)
				{
					_logger.LogError(ex, "Exception while running cleanup over installed tools: {Message}", ex.Message);
				}

				Task waitTask = cleanupTask.WaitAsync(cancellationToken);
				if (!success)
				{
					waitTask = Task.WhenAny(waitTask, Task.Delay(TimeSpan.FromMinutes(1.0), cancellationToken));
				}

				await waitTask;
			}
		}

		bool CleanupOnce()
		{
			bool result = true;

			List<DirectoryInfo> deleteToolDirs = new List<DirectoryInfo>();
			List<DirectoryInfo> deleteDeploymentDirs = new List<DirectoryInfo>();

			DirectoryInfo baseDirInfo = GetToolsDir().ToDirectoryInfo();
			if (baseDirInfo.Exists)
			{
				foreach (DirectoryInfo toolDir in baseDirInfo.EnumerateDirectories())
				{
					if (!toolDir.Name.StartsWith('.'))
					{
						DirectoryInfo[] deploymentDirs = toolDir.GetDirectories();

						ToolId? toolId;
						try
						{
							toolId = new ToolId(toolDir.Name);
						}
						catch
						{
							toolId = null;
						}

						lock (_workerThreadLockObject)
						{
							ItemState? itemState = null;
							if (toolId != null && _items.TryGetValue(toolId.Value, out Item? item))
							{
								itemState = item._state;
							}

							bool deleteToolDir = true;
							for (int idx = 0; idx < deploymentDirs.Length; idx++)
							{
								DirectoryInfo deploymentDir = deploymentDirs[idx];
								if (!deploymentDir.Name.StartsWith('.'))
								{
									if (!BinaryId.TryParse(deploymentDir.Name, out BinaryId deploymentId))
									{
										deleteDeploymentDirs.Add(deploymentDir);
									}
									else if (itemState != null && itemState.Current != null && itemState.Current.Id.Id == deploymentId)
									{
										deleteToolDir = false;
									}
									else if (itemState != null && itemState.Pending?.Deployment != null && itemState.Pending.Deployment.Id.Id == deploymentId)
									{
										deleteToolDir = false;
									}
									else if (TryRenameDir(deploymentDir, deploymentDir.FullName + ".dead"))
									{
										deleteDeploymentDirs.Add(deploymentDir);
									}
									else
									{
										result = deleteToolDir = false;
									}
								}
							}

							if (deleteToolDir)
							{
								deleteToolDirs.Add(toolDir);
							}
						}
					}
				}
			}

			// Try to delete all the deployment directories
			foreach (DirectoryInfo deleteDeploymentDir in deleteDeploymentDirs)
			{
				result &= ForceDeleteDir(deleteDeploymentDir);
			}

			// Try to delete all the tool directories if they're empty
			foreach (DirectoryInfo deleteToolDir in deleteToolDirs)
			{
				result &= TryDeleteDir(deleteToolDir);
			}

			return result;
		}

		bool TryRenameDir(DirectoryInfo dirInfo, string newName)
		{
			try
			{
				dirInfo.MoveTo(newName);
				return true;
			}
			catch (Exception ex)
			{
				_logger.LogWarning(ex, "Unable to rename directory {DirName}: {Message}", dirInfo.FullName, ex.Message);
				return false;
			}
		}

		bool ForceDeleteDir(DirectoryInfo dirInfo)
		{
			try
			{
				FileUtils.ForceDeleteDirectory(dirInfo);
				return true;
			}
			catch (Exception ex)
			{
				_logger.LogWarning(ex, "Unable to delete directory {DirName}: {Message}", dirInfo.FullName, ex.Message);
				return false;
			}
		}

		bool TryDeleteDir(DirectoryInfo dirInfo)
		{
			try
			{
				dirInfo.Delete(false);
				return true;
			}
			catch (Exception ex)
			{
				_logger.LogWarning(ex, "Unable to delete directory {DirName}: {Message}", dirInfo.FullName, ex.Message);
				return false;
			}
		}

		#endregion

		#region Updates

		async Task UpdateAsync(CancellationToken cancellationToken)
		{
			while (!cancellationToken.IsCancellationRequested)
			{
				Task updateTask = _updateEvent.Task;

				using (CancellationTokenSource cancellationSource = CancellationTokenSource.CreateLinkedTokenSource(cancellationToken))
				{
					bool complete = false;
					_ = updateTask.ContinueWith(_ =>
					{
						lock (cancellationSource)
						{
							if (!complete)
							{
								cancellationSource.Cancel();
							}
						}
					}, TaskScheduler.Default);

					try
					{
						await PollForUpdatesOnceAsync(cancellationSource.Token);
					}
					catch (OperationCanceledException)
					{
					}
					catch (InvalidCredentialException)
					{
						// Briefly wait and try again as missing access token check is cheap and in-memory.
						await Task.WhenAny(updateTask, Task.Delay(TimeSpan.FromSeconds(5), cancellationToken));
						continue; 
					}
					catch (Exception ex)
					{
						_logger.LogError(ex, "Exception while checking for tool updates: {Message}", ex.Message);
					}
					finally
					{
						lock (cancellationSource)
						{
							complete = true;
						}
					}
				}

				await Task.WhenAny(updateTask, Task.Delay(TimeSpan.FromMinutes(5.0), cancellationToken));
			}
		}

		async Task PollForUpdatesOnceAsync(CancellationToken cancellationToken)
		{
			using IHordeClientRef? hordeClientRef = _hordeClientProvider.GetClientRef();
			if (hordeClientRef == null)
			{
				return;
			}

			if (_serverUri != hordeClientRef.Client.ServerUrl)
			{
				SetTools(Enumerable.Empty<ITool>());
				_serverUri = hordeClientRef.Client.ServerUrl;
			}
			
			if (!hordeClientRef.Client.HasValidAccessToken())
			{
				throw new InvalidCredentialException("No access token available. Auth via Toolbox UI is required.");
			}

			IEnumerable<ITool> tools = await hordeClientRef.Client.Tools.GetAllAsync(cancellationToken);
			SetTools(tools);
		}

		void SetTools(IEnumerable<ITool> tools)
		{
			lock (_workerThreadLockObject)
			{
				Dictionary<ToolId, Item> newItems = new Dictionary<ToolId, Item>();
				foreach (ITool tool in tools)
				{
					if (ShouldIncludeTool(tool))
					{
						string? msiProductId;
						if (!tool.Metadata.TryGetValue("msi-product-id", out msiProductId))
						{
							msiProductId = null;
						}

						IToolDeployment? latestDeployment = (tool.Deployments.Count > 0) ? tool.Deployments[^1] : null;
						if (_items.TryGetValue(tool.Id, out Item? item))
						{
							item._state = new ItemState(tool.Name, tool.Description, msiProductId, item._state.Current, item._state.Pending, latestDeployment);
						}
						else
						{
							item = new Item(this, tool.Id, new ItemState(tool.Name, tool.Description, msiProductId, null, null, latestDeployment));
						}

						if (OperatingSystem.IsWindows() && item._state.Current != null && msiProductId != null && !IsMsiInstalled(msiProductId))
						{
							item._state = item._state with { Current = null };
						}

						newItems[item.Id] = item;
					}
				}
				foreach (Item existingItem in _items.Values)
				{
					if (existingItem._state.Current != null && !newItems.ContainsKey(existingItem.Id))
					{
						newItems.Add(existingItem.Id, existingItem);
					}
				}
				_items = newItems;

				if (_autoUpdate)
				{
					foreach (Item item in _items.Values)
					{
						if (item._state.Current != null && !item._state.Current.Config.ManualInstall && String.IsNullOrEmpty(item._state.MsiProductId))
						{
							if (item._state.Latest != null && item._state.Latest.Id != item._state.Current.Id)
							{
								UpdateDeployment(item, item._state.Latest);
							}
						}
					}
				}
			}
			PostMainThreadUpdate();
		}

		static IReadOnlySet<string> HostPlatformRids { get; } = GetHostPlatformRids();

		static bool ShouldIncludeTool(ITool tool)
		{
			if (!tool.ShowInToolbox)
			{
				return false;
			}
			if (tool.Platforms != null && !tool.Platforms.Any(x => HostPlatformRids.Contains(x)))
			{
				return false;
			}
			return true;
		}

		static HashSet<string> GetHostPlatformRids()
		{
			HashSet<string> platforms = new HashSet<string>(StringComparer.OrdinalIgnoreCase);
			platforms.Add("any");
			if (OperatingSystem.IsWindows())
			{
				platforms.Add("win");
				switch (RuntimeInformation.OSArchitecture)
				{
					case Architecture.X64:
						platforms.Add("win-x64");
						break;
					case Architecture.Arm64:
						platforms.Add("win-arm64");
						break;
				}
			}
			else if (OperatingSystem.IsMacOS())
			{
				platforms.Add("osx");
				switch (RuntimeInformation.OSArchitecture)
				{
					case Architecture.X64:
						platforms.Add("osx-x64");
						break;
					case Architecture.Arm64:
						platforms.Add("osx-arm64");
						break;
				}
			}
			else if (OperatingSystem.IsLinux())
			{
				platforms.Add("linux");
				platforms.Add("unix");
				switch (RuntimeInformation.OSArchitecture)
				{
					case Architecture.X64:
						platforms.Add("linux-x64");
						platforms.Add("unix-x64");
						break;
					case Architecture.Arm64:
						platforms.Add("linux-arm64");
						platforms.Add("unix-arm64");
						break;
				}
			}
			return platforms;
		}

		#endregion
	}
}
