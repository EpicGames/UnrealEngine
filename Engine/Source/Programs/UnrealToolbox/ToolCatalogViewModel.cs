// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.ObjectModel;
using System.ComponentModel;
using System.Diagnostics;
using System.Diagnostics.CodeAnalysis;
using CommunityToolkit.Mvvm.ComponentModel;

namespace UnrealToolbox
{
	partial class ToolCatalogViewModel : ObservableObject, IDisposable
	{
		readonly IToolCatalog? _catalog;
		readonly Dictionary<IToolCatalogItem, ToolCatalogItemViewModel> _items = new Dictionary<IToolCatalogItem, ToolCatalogItemViewModel>();

		public ObservableCollection<ToolCatalogItemViewModel> Items { get; } = new ObservableCollection<ToolCatalogItemViewModel>();

		public bool ShowTools => Items.Count > 0;

		[ObservableProperty]
		bool _autoUpdate = true;

		public ToolCatalogViewModel()
			: this(null)
		{ }

		public ToolCatalogViewModel(IToolCatalog? catalog)
		{
			if (catalog != null)
			{
				_catalog = catalog;
				_catalog.OnItemsChanged += RefreshItems;

				AutoUpdate = catalog.AutoUpdate;

				Items.CollectionChanged += (_, _) => OnPropertyChanged(nameof(ShowTools));

				RefreshItems();
			}
		}

		protected override void OnPropertyChanged(PropertyChangedEventArgs e)
		{
			base.OnPropertyChanged(e);

			if (_catalog != null)
			{
				_catalog.AutoUpdate = AutoUpdate;
			}
		}

		public void Dispose()
		{
			foreach (ToolCatalogItemViewModel item in _items.Values)
			{
				item.Dispose();
			}
			if (_catalog != null)
			{
				_catalog.OnItemsChanged -= RefreshItems;
			}
		}

		void RefreshItems()
		{
			if (_catalog != null)
			{
				Items.Clear();

				HashSet<IToolCatalogItem> newItemSet = new HashSet<IToolCatalogItem>(_catalog.Items);
				foreach ((IToolCatalogItem item, ToolCatalogItemViewModel itemViewModel) in _items.ToArray())
				{
					if (!newItemSet.Contains(item))
					{
						_items.Remove(item);
						itemViewModel.Dispose();
					}
				}

				foreach (IToolCatalogItem item in _catalog.Items)
				{
					ToolCatalogItemViewModel? itemViewModel;
					if (!_items.TryGetValue(item, out itemViewModel))
					{
						itemViewModel = new ToolCatalogItemViewModel(item);
						_items.Add(item, itemViewModel);
					}
					Items.Add(itemViewModel);
				}
			}
		}
	}

	partial class ToolCatalogItemViewModel : ObservableObject, IDisposable
	{
		readonly IToolCatalogItem _item;

		public ToolCatalogItemViewModel(IToolCatalogItem item)
		{
			_item = item;
			_item.OnItemChanged += UpdateFromModel;
			UpdateFromModel();
		}

		public void Dispose()
		{
			_item.OnItemChanged -= UpdateFromModel;
		}

		[ObservableProperty]
		string _name = String.Empty;

		[ObservableProperty]
		string _description = String.Empty;

		[ObservableProperty]
		string _status = String.Empty;

		[ObservableProperty]
		bool _showStatusLink;

		[ObservableProperty]
		bool _showStatusText;

		[ObservableProperty]
		bool _showAdd;

		[ObservableProperty]
		bool _showUpdate;

		[ObservableProperty]
		bool _showRemove;

		[ObservableProperty]
		bool _isBusy;

		internal void UpdateFromModel()
		{
			Name = _item.Name;
			Description = _item.Description;

			PendingToolDeploymentInfo? pending = _item.Pending;
			if (pending != null)
			{
				if (pending.Message != null)
				{
					Status = pending.Message;
				}
				else if (pending.Deployment == null)
				{
					Status = "Removing...";
				}
				else
				{
					Status = $"Updating to {pending.Deployment.Version}...";
				}
				IsBusy = !pending.Failed;
			}
			else
			{
				ToolDeploymentInfo? current = _item.Current;
				if (current != null)
				{
					Status = current.Version;
				}
				else
				{
					Status = String.Empty;
				}
				IsBusy = false;
			}

			ShowStatusLink = _item.Pending != null && _item.Pending.ShowLogLink;
			ShowStatusText = !String.IsNullOrEmpty(Status) && !ShowStatusLink;
			ShowAdd = _item.Latest != null && _item.Current == null;
			ShowUpdate = _item.Latest != null && _item.Current != null && _item.Latest.Id != _item.Current.Id;
			ShowRemove = _item.Current != null;
		}
		
		[SuppressMessage("Performance", "CA1822:Mark members as static", Justification = "Avalonia cannot call static methods as a binding")]
		public void ShowLog()
		{
			if (OperatingSystem.IsWindows() && Program.LogFile != null)
			{
				Process.Start(new ProcessStartInfo(Program.LogFile.FullName) { UseShellExecute = true });
			}
		}

		public void Add()
			=> _item.Install();

		public void Remove()
			=> _item.Uninstall();

		public void Update()
			=> _item.Install();
	}
}
