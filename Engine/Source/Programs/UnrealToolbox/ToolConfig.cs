// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealToolbox
{
	/// <summary>
	/// Configuration for a tool that can be shown in the launcher
	/// </summary>
	public class ToolConfig
	{
		/// <summary>
		/// Popup menu item for the tool
		/// </summary>
		public ToolMenuItem? PopupMenu { get; set; }

		/// <summary>
		/// Whether the install/uninstall for this tool needs to be triggered manually
		/// </summary>
		public bool ManualInstall { get; set; }
		
		/// <summary>
		/// Whether the install/uninstall for this tool needs administrator or root privileges
		/// </summary>
		public bool RequiresElevation { get; set; } = false;
		
		/// <summary>
		/// Whether the install/uninstall process window should be hidden
		/// </summary>
		public bool Hidden { get; set; } = false;

		/// <summary>
		/// Command to run when installing
		/// </summary>
		public ToolCommand? InstallCommand { get; set; }

		/// <summary>
		/// Command to run when uninstalling
		/// </summary>
		public ToolCommand? UninstallCommand { get; set; }
	}

	/// <summary>
	/// Menu item for the tool
	/// </summary>
	public class ToolMenuItem
	{
		/// <summary>
		/// Text to display for the menu item. If null, or a sequence of hyphens, will create a menu separator.
		/// </summary>
		public string? Label { get; set; }

		/// <summary>
		/// Command to run when the menu item is clicked. Will be executed in the root directory of the downloaded tool.
		/// </summary>
		public ToolCommand? Command { get; set; }

		/// <summary>
		/// Child menu items. Cannot be specified at the same time as a tool.
		/// </summary>
		public List<ToolMenuItem>? Children { get; set; }
	}

	/// <summary>
	/// Command to run for a menu item.
	/// </summary>
	public class ToolCommand
	{
		/// <summary>
		/// Executable to run
		/// </summary>
		public string FileName { get; set; } = "cmd.exe";

		/// <summary>
		/// Command line arguments for the tool
		/// </summary>
		public List<string>? Arguments { get; set; }
	}
}
