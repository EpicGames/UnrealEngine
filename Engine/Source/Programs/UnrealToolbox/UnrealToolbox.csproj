<Project Sdk="Microsoft.NET.Sdk">
  <Import Project="..\Shared\UnrealEngine.csproj.props" />

  <ItemGroup>
    <FrameworkReference Include="Microsoft.AspNetCore.App" />
  </ItemGroup>

  <PropertyGroup>
    <OutputType>WinExe</OutputType>
    <Nullable>enable</Nullable>
    <ImplicitUsings>enable</ImplicitUsings>
    <NeutralLanguage>en-US</NeutralLanguage>
    <AssemblyName>UnrealToolbox</AssemblyName>
    <AvaloniaUseCompiledBindingsByDefault>true</AvaloniaUseCompiledBindingsByDefault>
    <NoWarn>$(NoWarn);CA1501</NoWarn>
    <AllowUnsafeBlocks>true</AllowUnsafeBlocks>
    <ApplicationIcon>Resources/StatusNormal.ico</ApplicationIcon>
  </PropertyGroup>
  
  <PropertyGroup Condition="'$(EnableForTPS)'=='true'">
     <TargetFramework>net10.0-windows</TargetFramework>
   </PropertyGroup>

  <PropertyGroup Condition="'$(EnableForTPS)'!='true'">
     <TargetFramework>net10.0-windows10.0.17763.0</TargetFramework>
     <WindowsSdkPackageVersion>10.0.17763.41</WindowsSdkPackageVersion>
   </PropertyGroup>
  
  <PropertyGroup Condition=" '$(Configuration)|$(Platform)' == 'Analyze|AnyCPU' ">
    <RunAnalyzersDuringBuild>True</RunAnalyzersDuringBuild>
    <TreatWarningsAsErrors>false</TreatWarningsAsErrors>
    <EnforceCodeStyleInBuild>true</EnforceCodeStyleInBuild>
    <GenerateDocumentationFile>true</GenerateDocumentationFile>
    <DocumentationFile></DocumentationFile>
  </PropertyGroup>
  <ItemGroup>
    <Compile Include="..\Horde\HordeAgent\IpcMessages.cs" Link="Plugins\HordeAgent\IpcMessages.cs" />
  </ItemGroup>

  <ItemGroup>
    <ProjectReference Include="..\Shared\EpicGames.Core\EpicGames.Core.csproj" />
    <ProjectReference Include="..\Shared\EpicGames.Horde\EpicGames.Horde.csproj" />
  </ItemGroup>

  <ItemGroup>
    <Compile Update="GeneralSettingsPage.axaml.cs">
      <DependentUpon>GeneralSettingsPage.axaml</DependentUpon>
    </Compile>
    <Compile Update="Plugins\HordeAgent\HordeAgentSettingsPage.axaml.cs">
      <DependentUpon>HordeAgentSettingsPage.axaml</DependentUpon>
    </Compile>
    <Compile Update="Properties\Resources.Designer.cs">
      <DesignTime>True</DesignTime>
      <AutoGen>True</AutoGen>
      <DependentUpon>Resources.resx</DependentUpon>
    </Compile>
    <Compile Update="HordeServerSettings.axaml.cs">
      <DependentUpon>HordeServerSettings.axaml</DependentUpon>
    </Compile>
    <Compile Update="SettingsWindow.axaml.cs">
      <DependentUpon>SettingsWindow.axaml</DependentUpon>
    </Compile>
    <Compile Update="AboutPage.axaml.cs">
      <DependentUpon>AboutPage.axaml</DependentUpon>
    </Compile>
  </ItemGroup>

  <ItemGroup>
    <EmbeddedResource Update="Properties\Resources.resx">
      <Generator>ResXFileCodeGenerator</Generator>
      <LastGenOutput>Resources.Designer.cs</LastGenOutput>
    </EmbeddedResource>
  </ItemGroup>
  
  <ItemGroup>
    <AvaloniaResource Include="Resources\**" />
  </ItemGroup>

  <ItemGroup>
    <PackageReference Include="Avalonia" Version="11.1.5" />
    <PackageReference Include="Avalonia.Desktop" Version="11.1.5" />
    <PackageReference Include="Avalonia.Themes.Fluent" Version="11.1.5" />
    <PackageReference Include="Avalonia.Fonts.Inter" Version="11.1.5" />
    <!--Condition below is needed to remove Avalonia.Diagnostics package from build output in Release configuration.-->
    <PackageReference Condition="'$(Configuration)' == 'Debug'" Include="Avalonia.Diagnostics" Version="11.1.5" />
    <PackageReference Include="FluentAvaloniaUI" Version="2.1.0" />
    <PackageReference Include="CommunityToolkit.Mvvm" Version="8.3.2" />
    <PackageReference Include="DesktopNotifications.Avalonia" Version="1.3.1" />
    <PackageReference Include="System.Drawing.Common" Version="10.0.6" />
  </ItemGroup>

</Project>